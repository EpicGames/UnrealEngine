{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Network Service Discovery",
	"Description": "Cross-platform mDNS/DNS-SD service discovery using native platform APIs (NSNetServiceBrowser on Apple, NsdManager on Android, native DNS-SD via dnsapi on Windows)",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "NetworkServiceDiscovery",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"IOS",
				"Android"
			]
		}
	]
}
