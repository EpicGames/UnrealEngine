// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class NetworkServiceDiscovery : ModuleRules
{
	public NetworkServiceDiscovery(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bRequiresPlatformSDK = true;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateDependencyModuleNames.Add("Launch");

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin",
				Path.Combine(PluginPath, "NetworkServiceDiscovery_APL_Android.xml"));
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicDefinitions.Add("WITH_WINDOWS_DNSSD=1");
			// dnsapi.dll is loaded dynamically at runtime — no link-time dependency.
			// This allows graceful fallback on Windows versions that lack DNS-SD APIs.
		}
		else
		{
			PublicDefinitions.Add("WITH_WINDOWS_DNSSD=0");
		}
	}
}
