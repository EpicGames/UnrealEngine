<?xml version="1.0" encoding="utf-8"?>
<!--NetworkServiceDiscovery plugin additions for Android-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<init>
		<log text="NetworkServiceDiscovery Android init"/>
	</init>

	<prebuildCopies>
		<log text="Copying NetworkServiceDiscoveryHelper java file"/>
		<copyFile src="$S(PluginDir)/Private/Android/Java/NetworkServiceDiscoveryHelper.java"
				  dst="$S(BuildDir)/src/com/epicgames/unreal/NetworkServiceDiscoveryHelper.java" />
	</prebuildCopies>

	<proguardAdditions>
		<insert>
-keep class com.epicgames.unreal.NetworkServiceDiscoveryHelper {
	public *;
	private static native *;
}
		</insert>
	</proguardAdditions>

	<gameActivityImportAdditions>
		<insert>
import com.epicgames.unreal.NetworkServiceDiscoveryHelper;
		</insert>
	</gameActivityImportAdditions>

	<gameActivityOnCreateAdditions>
		<insert>
		NetworkServiceDiscoveryHelper.Initialize(getApplicationContext());
		</insert>
	</gameActivityOnCreateAdditions>
</root>
