// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.content.Context;
import android.net.nsd.NsdManager;
import android.net.nsd.NsdServiceInfo;
import android.os.Build;
import android.util.Log;

import java.net.Inet4Address;
import java.net.InetAddress;
import java.net.Inet6Address;
import java.net.NetworkInterface;
import java.util.Enumeration;
import java.util.HashMap;
import java.util.Map;
import java.util.concurrent.ConcurrentHashMap;

/**
 * Java helper for Android NSD (Network Service Discovery) via NsdManager.
 * Called from C++ via JNI, and calls back to C++ via native methods.
 * Supports multiple simultaneous service registrations.
 */
public class NetworkServiceDiscoveryHelper
{
	private static final String TAG = "NSDHelper";

	private static NetworkServiceDiscoveryHelper Instance;

	private Context AppContext;
	private NsdManager NsdMgr;

	/** Active registrations keyed by service name */
	private ConcurrentHashMap<String, NsdManager.RegistrationListener> RegistrationListeners = new ConcurrentHashMap<>();

	private NsdManager.DiscoveryListener DiscoveryListener;
	private boolean bIsDiscovering = false;

	// ------------------------------------------------------------------
	//  Native callbacks (implemented in C++)
	// ------------------------------------------------------------------

	private static native void nativeOnServiceFound(String serviceName, String serviceType);
	private static native void nativeOnServiceLost(String serviceName, String serviceType);
	private static native void nativeOnServiceResolved(String serviceName, String serviceType,
		String hostAddress, int port, String[] txtKeys, String[] txtValues);
	private static native void nativeOnServiceRegistered(String serviceName, String serviceType, int port);
	private static native void nativeOnError(String errorMessage);

	// ------------------------------------------------------------------
	//  Initialization
	// ------------------------------------------------------------------

	public static void Initialize(Context context)
	{
		if (Instance == null)
		{
			Instance = new NetworkServiceDiscoveryHelper();
			Instance.AppContext = context.getApplicationContext();
			Instance.NsdMgr = (NsdManager) Instance.AppContext.getSystemService(Context.NSD_SERVICE);
			Log.d(TAG, "NetworkServiceDiscoveryHelper initialized");
		}
	}

	public static NetworkServiceDiscoveryHelper GetInstance()
	{
		return Instance;
	}

	// ------------------------------------------------------------------
	//  Service Registration (supports multiple simultaneous registrations)
	// ------------------------------------------------------------------

	public boolean registerService(String serviceName, String serviceType, int port,
		String[] txtKeys, String[] txtValues)
	{
		if (NsdMgr == null)
		{
			nativeOnError("NsdManager not available");
			return false;
		}

		// Unregister any existing service with the same name
		unregisterService(serviceName);

		NsdServiceInfo serviceInfo = new NsdServiceInfo();
		serviceInfo.setServiceName(serviceName);
		serviceInfo.setServiceType(serviceType);
		serviceInfo.setPort(port);

		if (txtKeys != null && txtValues != null)
		{
			for (int i = 0; i < txtKeys.length && i < txtValues.length; i++)
			{
				serviceInfo.setAttribute(txtKeys[i], txtValues[i]);
			}
		}

		final String regName = serviceName;
		final int regPort = port;

		NsdManager.RegistrationListener listener = new NsdManager.RegistrationListener()
		{
			@Override
			public void onServiceRegistered(NsdServiceInfo info)
			{
				Log.d(TAG, "Service registered: " + info.getServiceName());
				nativeOnServiceRegistered(info.getServiceName(), info.getServiceType(), regPort);
			}

			@Override
			public void onRegistrationFailed(NsdServiceInfo info, int errorCode)
			{
				Log.e(TAG, "Registration failed for '" + regName + "': " + errorCode);
				RegistrationListeners.remove(regName);
				nativeOnError("NSD registration failed for '" + regName + "' with error code: " + errorCode);
			}

			@Override
			public void onServiceUnregistered(NsdServiceInfo info)
			{
				Log.d(TAG, "Service unregistered: " + info.getServiceName());
				RegistrationListeners.remove(regName);
			}

			@Override
			public void onUnregistrationFailed(NsdServiceInfo info, int errorCode)
			{
				Log.e(TAG, "Unregistration failed for '" + regName + "': " + errorCode);
				nativeOnError("NSD unregistration failed for '" + regName + "' with error code: " + errorCode);
			}
		};

		try
		{
			NsdMgr.registerService(serviceInfo, NsdManager.PROTOCOL_DNS_SD, listener);
			RegistrationListeners.put(serviceName, listener);
			return true;
		}
		catch (Exception e)
		{
			Log.e(TAG, "Exception registering service: " + e.getMessage());
			nativeOnError("Exception registering service: " + e.getMessage());
			return false;
		}
	}

	public void unregisterService(String serviceName)
	{
		if (NsdMgr == null) return;

		if (serviceName == null || serviceName.isEmpty())
		{
			// Unregister all
			for (Map.Entry<String, NsdManager.RegistrationListener> entry : RegistrationListeners.entrySet())
			{
				try
				{
					NsdMgr.unregisterService(entry.getValue());
				}
				catch (Exception e)
				{
					Log.w(TAG, "Exception unregistering service '" + entry.getKey() + "': " + e.getMessage());
				}
			}
			RegistrationListeners.clear();
		}
		else
		{
			NsdManager.RegistrationListener listener = RegistrationListeners.remove(serviceName);
			if (listener != null)
			{
				try
				{
					NsdMgr.unregisterService(listener);
				}
				catch (Exception e)
				{
					Log.w(TAG, "Exception unregistering service '" + serviceName + "': " + e.getMessage());
				}
			}
		}
	}

	public boolean isServiceRegistered(String serviceName)
	{
		if (serviceName == null || serviceName.isEmpty())
		{
			return !RegistrationListeners.isEmpty();
		}
		return RegistrationListeners.containsKey(serviceName);
	}

	// ------------------------------------------------------------------
	//  Service Discovery
	// ------------------------------------------------------------------

	public boolean startDiscovery(String serviceType)
	{
		if (NsdMgr == null)
		{
			nativeOnError("NsdManager not available");
			return false;
		}

		stopDiscovery();

		DiscoveryListener = new NsdManager.DiscoveryListener()
		{
			@Override
			public void onDiscoveryStarted(String regType)
			{
				Log.d(TAG, "Discovery started for: " + regType);
				bIsDiscovering = true;
			}

			@Override
			public void onServiceFound(NsdServiceInfo info)
			{
				Log.d(TAG, "Service found: " + info.getServiceName() + " type: " + info.getServiceType());
				nativeOnServiceFound(info.getServiceName(), info.getServiceType());
			}

			@Override
			public void onServiceLost(NsdServiceInfo info)
			{
				Log.d(TAG, "Service lost: " + info.getServiceName());
				nativeOnServiceLost(info.getServiceName(), info.getServiceType());
			}

			@Override
			public void onDiscoveryStopped(String serviceType)
			{
				Log.d(TAG, "Discovery stopped for: " + serviceType);
				bIsDiscovering = false;
			}

			@Override
			public void onStartDiscoveryFailed(String serviceType, int errorCode)
			{
				Log.e(TAG, "Start discovery failed: " + errorCode);
				bIsDiscovering = false;
				nativeOnError("NSD start discovery failed with error code: " + errorCode);
			}

			@Override
			public void onStopDiscoveryFailed(String serviceType, int errorCode)
			{
				Log.e(TAG, "Stop discovery failed: " + errorCode);
				nativeOnError("NSD stop discovery failed with error code: " + errorCode);
			}
		};

		try
		{
			NsdMgr.discoverServices(serviceType, NsdManager.PROTOCOL_DNS_SD, DiscoveryListener);
			return true;
		}
		catch (Exception e)
		{
			Log.e(TAG, "Exception starting discovery: " + e.getMessage());
			nativeOnError("Exception starting discovery: " + e.getMessage());
			DiscoveryListener = null;
			return false;
		}
	}

	public void stopDiscovery()
	{
		if (DiscoveryListener != null && NsdMgr != null)
		{
			try
			{
				NsdMgr.stopServiceDiscovery(DiscoveryListener);
			}
			catch (Exception e)
			{
				Log.w(TAG, "Exception stopping discovery: " + e.getMessage());
			}
			DiscoveryListener = null;
		}
		bIsDiscovering = false;
	}

	public boolean isDiscovering()
	{
		return bIsDiscovering;
	}

	// ------------------------------------------------------------------
	//  Route-check helpers
	// ------------------------------------------------------------------

	/**
	 * Returns true if the device has at least one routable (non-link-local, non-APIPA)
	 * IPv4 address on any active network interface.
	 */
	private static boolean hasRoutableIPv4Address()
	{
		try
		{
			Enumeration<NetworkInterface> interfaces = NetworkInterface.getNetworkInterfaces();
			while (interfaces != null && interfaces.hasMoreElements())
			{
				NetworkInterface iface = interfaces.nextElement();
				if (!iface.isUp() || iface.isLoopback())
				{
					continue;
				}

				Enumeration<InetAddress> addrs = iface.getInetAddresses();
				while (addrs.hasMoreElements())
				{
					InetAddress addr = addrs.nextElement();
					if (addr instanceof Inet4Address && !addr.isLoopbackAddress() && !addr.isLinkLocalAddress())
					{
						Log.d(TAG, "Device has routable IPv4 address: " + addr.getHostAddress() + " on " + iface.getName());
						return true;
					}
				}
			}
		}
		catch (Exception e)
		{
			Log.w(TAG, "Failed to enumerate network interfaces: " + e.getMessage());
			return true; // Assume IPv4 is available if we can't check
		}

		Log.d(TAG, "Device has no routable IPv4 address");
		return false;
	}

	/**
	 * Finds the best routable address from a resolved service. On API 34+
	 * (Android 14), NsdServiceInfo provides all addresses and we probe each
	 * one. On older APIs only a single address is available so we use it as-is.
	 */
	private static InetAddress findRoutableAddress(NsdServiceInfo info)
	{
		if (Build.VERSION.SDK_INT >= 34)
		{
			return findRoutableAddressApi34(info);
		}

		// Pre-API 34: only one address available
		InetAddress host = info.getHost();
		if (host != null)
		{
			if (host instanceof Inet6Address && host.isLinkLocalAddress())
			{
				Log.d(TAG, "Skipping IPv6 " + host.getHostAddress() + " — link-local (pre-API 34)");
				return null;
			}
			String addrType = (host instanceof Inet6Address) ? "IPv6" : "IPv4";
			Log.d(TAG, "Candidate " + addrType + " " + host.getHostAddress() + " (single address, pre-API 34)");
		}
		return host;
	}

	/**
	 * API 34+ path: iterates all resolved addresses and picks the best one.
	 * Skips IPv4 candidates if the device has no routable IPv4 address.
	 * Skips IPv6 link-local addresses. Returns the first viable candidate.
	 */
	private static InetAddress findRoutableAddressApi34(NsdServiceInfo info)
	{
		java.util.List<InetAddress> addresses = info.getHostAddresses();
		boolean bHasRoutableIPv4 = hasRoutableIPv4Address();

		// First pass: log all candidates
		Log.d(TAG, "Service resolved with " + addresses.size() + " address(es) (device has routable IPv4: " + bHasRoutableIPv4 + "):");
		for (InetAddress addr : addresses)
		{
			String addrType = (addr instanceof Inet6Address) ? "IPv6" : "IPv4";
			boolean linkLocal = addr.isLinkLocalAddress();
			Log.d(TAG, "  " + addrType + " " + addr.getHostAddress() + (linkLocal ? " (link-local)" : ""));
		}

		// Second pass: pick the first viable address
		for (InetAddress candidate : addresses)
		{
			// Skip IPv6 link-local addresses
			if (candidate instanceof Inet6Address && candidate.isLinkLocalAddress())
			{
				continue;
			}

			// Skip IPv4 candidates if the device has no routable IPv4
			if (candidate instanceof Inet4Address && !bHasRoutableIPv4)
			{
				Log.d(TAG, "Skipping IPv4 " + candidate.getHostAddress() + " — device has no routable IPv4");
				continue;
			}

			String addrType = (candidate instanceof Inet6Address) ? "IPv6" : "IPv4";
			Log.d(TAG, "Selected " + addrType + " " + candidate.getHostAddress());
			return candidate;
		}

		// No viable address found — fall back to legacy single-address API
		Log.w(TAG, "No viable address found, falling back to " + info.getHost());
		return info.getHost();
	}

	// ------------------------------------------------------------------
	//  Service Resolution
	// ------------------------------------------------------------------

	public void resolveService(String serviceName, String serviceType)
	{
		if (NsdMgr == null)
		{
			nativeOnError("NsdManager not available for resolution");
			return;
		}

		NsdServiceInfo serviceInfo = new NsdServiceInfo();
		serviceInfo.setServiceName(serviceName);
		serviceInfo.setServiceType(serviceType);

		NsdMgr.resolveService(serviceInfo, new NsdManager.ResolveListener()
		{
			@Override
			public void onResolveFailed(NsdServiceInfo info, int errorCode)
			{
				Log.e(TAG, "Resolve failed for " + info.getServiceName() + ": " + errorCode);
				nativeOnError("NSD resolve failed for '" + info.getServiceName() + "' with error code: " + errorCode);
			}

			@Override
			public void onServiceResolved(NsdServiceInfo info)
			{
				Log.d(TAG, "Resolved: " + info.getServiceName() + " -> " + info.getHost() + ":" + info.getPort());

				InetAddress routableHost = findRoutableAddress(info);
				String hostAddress = (routableHost != null) ? routableHost.getHostAddress() : "";

				Map<String, byte[]> attributes = info.getAttributes();
				String[] keys = new String[attributes.size()];
				String[] values = new String[attributes.size()];
				int index = 0;
				for (Map.Entry<String, byte[]> entry : attributes.entrySet())
				{
					keys[index] = entry.getKey();
					byte[] valueBytes = entry.getValue();
					values[index] = (valueBytes != null) ? new String(valueBytes, java.nio.charset.StandardCharsets.UTF_8) : "";
					index++;
				}

				nativeOnServiceResolved(
					info.getServiceName(),
					info.getServiceType(),
					hostAddress,
					info.getPort(),
					keys,
					values
				);
			}
		});
	}
}
