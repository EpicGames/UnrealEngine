// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/NetworkServiceDiscoveryAndroid.h"
#include "NetworkServiceDiscoveryModule.h"
#include "Async/Async.h"

#if PLATFORM_ANDROID

#include "Android/AndroidApplication.h"
#include "Android/AndroidJNI.h"

FNetworkServiceDiscoveryAndroid* FNetworkServiceDiscoveryAndroid::Instance = nullptr;

FNetworkServiceDiscoveryAndroid::FNetworkServiceDiscoveryAndroid()
{
	Instance = this;
	InitJNI();
}

FNetworkServiceDiscoveryAndroid::~FNetworkServiceDiscoveryAndroid()
{
	if (bJNIInitialized)
	{
		UnregisterService(FString());
		StopDiscovery();
	}

	// Clean up any JNI global refs that were created (even if InitJNI failed partway)
	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		if (HelperInstance)
		{
			Env->DeleteGlobalRef(HelperInstance);
			HelperInstance = nullptr;
		}
		if (HelperClass)
		{
			Env->DeleteGlobalRef(HelperClass);
			HelperClass = nullptr;
		}
	}

	if (Instance == this)
	{
		Instance = nullptr;
	}
}

bool FNetworkServiceDiscoveryAndroid::InitJNI()
{
	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (!Env)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Android: Failed to get JNI environment");
		return false;
	}

	// Find our helper class
	jclass LocalClass = FAndroidApplication::FindJavaClass("com/epicgames/unreal/NetworkServiceDiscoveryHelper");
	if (!LocalClass)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Android: Failed to find NetworkServiceDiscoveryHelper class");
		Env->ExceptionClear();
		return false;
	}
	HelperClass = (jclass)Env->NewGlobalRef(LocalClass);
	Env->DeleteLocalRef(LocalClass);

	// Initialize the helper with the application context
	jmethodID InitMethod = Env->GetStaticMethodID(HelperClass, "Initialize", "(Landroid/content/Context;)V");
	Env->ExceptionClear();
	if (InitMethod)
	{
		jobject Activity = FAndroidApplication::GetGameActivityThis();
		Env->CallStaticVoidMethod(HelperClass, InitMethod, Activity);
		Env->ExceptionClear();
	}

	// Get the singleton instance
	jmethodID GetInstanceMethod = Env->GetStaticMethodID(HelperClass, "GetInstance", "()Lcom/epicgames/unreal/NetworkServiceDiscoveryHelper;");
	Env->ExceptionClear();
	if (GetInstanceMethod)
	{
		jobject LocalInstance = Env->CallStaticObjectMethod(HelperClass, GetInstanceMethod);
		Env->ExceptionClear();
		if (LocalInstance)
		{
			HelperInstance = Env->NewGlobalRef(LocalInstance);
			Env->DeleteLocalRef(LocalInstance);
		}
	}

	if (!HelperInstance)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Android: Failed to get helper instance");
		// Clean up HelperClass global ref since we're failing
		Env->DeleteGlobalRef(HelperClass);
		HelperClass = nullptr;
		return false;
	}

	// Cache method IDs — validate each one; a null method ID would crash on use
	RegisterServiceMethod = Env->GetMethodID(HelperClass, "registerService",
		"(Ljava/lang/String;Ljava/lang/String;I[Ljava/lang/String;[Ljava/lang/String;)Z");
	Env->ExceptionClear();
	UnregisterServiceMethod = Env->GetMethodID(HelperClass, "unregisterService", "(Ljava/lang/String;)V");
	Env->ExceptionClear();
	StartDiscoveryMethod = Env->GetMethodID(HelperClass, "startDiscovery", "(Ljava/lang/String;)Z");
	Env->ExceptionClear();
	StopDiscoveryMethod = Env->GetMethodID(HelperClass, "stopDiscovery", "()V");
	Env->ExceptionClear();
	ResolveServiceMethod = Env->GetMethodID(HelperClass, "resolveService",
		"(Ljava/lang/String;Ljava/lang/String;)V");
	Env->ExceptionClear();
	IsServiceRegisteredMethod = Env->GetMethodID(HelperClass, "isServiceRegistered", "(Ljava/lang/String;)Z");
	Env->ExceptionClear();
	IsDiscoveringMethod = Env->GetMethodID(HelperClass, "isDiscovering", "()Z");
	Env->ExceptionClear();

	if (!RegisterServiceMethod || !UnregisterServiceMethod || !StartDiscoveryMethod ||
		!StopDiscoveryMethod || !ResolveServiceMethod || !IsServiceRegisteredMethod || !IsDiscoveringMethod)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Android: Failed to resolve one or more JNI method IDs. Check ProGuard rules and Java method signatures.");
		// Clean up global refs since we're failing
		Env->DeleteGlobalRef(HelperInstance);
		HelperInstance = nullptr;
		Env->DeleteGlobalRef(HelperClass);
		HelperClass = nullptr;
		return false;
	}

	bJNIInitialized = true;
	UE_LOGF(LogNetworkServiceDiscovery, Log, "Android: JNI initialized successfully");
	return true;
}

// --- Registration ---

bool FNetworkServiceDiscoveryAndroid::RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord)
{
	if (!bJNIInitialized)
	{
		return false;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (!Env)
	{
		return false;
	}

	jstring JName = Env->NewStringUTF(TCHAR_TO_UTF8(*ServiceName));
	jstring JType = Env->NewStringUTF(TCHAR_TO_UTF8(*ServiceType));

	// Build TXT record arrays
	jobjectArray JKeys = nullptr;
	jobjectArray JValues = nullptr;

	if (TxtRecord.Num() > 0)
	{
		jclass StringClass = Env->FindClass("java/lang/String");
		if (StringClass)
		{
			JKeys = Env->NewObjectArray(TxtRecord.Num(), StringClass, nullptr);
			JValues = Env->NewObjectArray(TxtRecord.Num(), StringClass, nullptr);

			if (JKeys && JValues)
			{
				int32 Index = 0;
				for (const auto& Pair : TxtRecord)
				{
					jstring JKey = Env->NewStringUTF(TCHAR_TO_UTF8(*Pair.Key));
					jstring JValue = Env->NewStringUTF(TCHAR_TO_UTF8(*Pair.Value));
					Env->SetObjectArrayElement(JKeys, Index, JKey);
					Env->SetObjectArrayElement(JValues, Index, JValue);
					Env->DeleteLocalRef(JKey);
					Env->DeleteLocalRef(JValue);
					Index++;
				}
			}
			Env->DeleteLocalRef(StringClass);
		}
	}

	jboolean Result = Env->CallBooleanMethod(HelperInstance, RegisterServiceMethod, JName, JType, (jint)Port, JKeys, JValues);

	if (Env->ExceptionCheck())
	{
		Env->ExceptionClear();
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Android: Exception in RegisterService");
		Result = JNI_FALSE;
	}

	Env->DeleteLocalRef(JName);
	Env->DeleteLocalRef(JType);
	if (JKeys) Env->DeleteLocalRef(JKeys);
	if (JValues) Env->DeleteLocalRef(JValues);

	return (bool)Result;
}

void FNetworkServiceDiscoveryAndroid::UnregisterService(const FString& ServiceName)
{
	if (!bJNIInitialized)
	{
		return;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		jstring JName = Env->NewStringUTF(TCHAR_TO_UTF8(*ServiceName));
		Env->CallVoidMethod(HelperInstance, UnregisterServiceMethod, JName);
		if (Env->ExceptionCheck())
		{
			Env->ExceptionClear();
			UE_LOGF(LogNetworkServiceDiscovery, Warning, "Android: Exception in UnregisterService");
		}
		Env->DeleteLocalRef(JName);
	}
}

bool FNetworkServiceDiscoveryAndroid::IsServiceRegistered(const FString& ServiceName) const
{
	if (!bJNIInitialized)
	{
		return false;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		jstring JName = Env->NewStringUTF(TCHAR_TO_UTF8(*ServiceName));
		jboolean Result = Env->CallBooleanMethod(HelperInstance, IsServiceRegisteredMethod, JName);
		Env->DeleteLocalRef(JName);
		return (bool)Result;
	}
	return false;
}

// --- Discovery ---

bool FNetworkServiceDiscoveryAndroid::StartDiscovery(const FString& ServiceType)
{
	if (!bJNIInitialized)
	{
		return false;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (!Env)
	{
		return false;
	}

	jstring JType = Env->NewStringUTF(TCHAR_TO_UTF8(*ServiceType));
	jboolean Result = Env->CallBooleanMethod(HelperInstance, StartDiscoveryMethod, JType);

	if (Env->ExceptionCheck())
	{
		Env->ExceptionClear();
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Android: Exception in StartDiscovery");
		Result = JNI_FALSE;
	}
	Env->DeleteLocalRef(JType);

	return (bool)Result;
}

void FNetworkServiceDiscoveryAndroid::StopDiscovery()
{
	if (!bJNIInitialized)
	{
		return;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		Env->CallVoidMethod(HelperInstance, StopDiscoveryMethod);
		if (Env->ExceptionCheck())
		{
			Env->ExceptionClear();
			UE_LOGF(LogNetworkServiceDiscovery, Warning, "Android: Exception in StopDiscovery");
		}
	}

	FScopeLock Lock(&ServicesLock);
	DiscoveredServices.Empty();
}

bool FNetworkServiceDiscoveryAndroid::IsDiscovering() const
{
	if (!bJNIInitialized)
	{
		return false;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		return (bool)Env->CallBooleanMethod(HelperInstance, IsDiscoveringMethod);
	}
	return false;
}

void FNetworkServiceDiscoveryAndroid::ResolveService(const FNetworkServiceInfo& Service)
{
	if (!bJNIInitialized)
	{
		return;
	}

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (!Env)
	{
		return;
	}

	jstring JName = Env->NewStringUTF(TCHAR_TO_UTF8(*Service.ServiceName));
	jstring JType = Env->NewStringUTF(TCHAR_TO_UTF8(*Service.ServiceType));
	Env->CallVoidMethod(HelperInstance, ResolveServiceMethod, JName, JType);
	if (Env->ExceptionCheck())
	{
		Env->ExceptionClear();
		UE_LOGF(LogNetworkServiceDiscovery, Warning, "Android: Exception in ResolveService");
	}
	Env->DeleteLocalRef(JName);
	Env->DeleteLocalRef(JType);
}

TArray<FNetworkServiceInfo> FNetworkServiceDiscoveryAndroid::GetDiscoveredServices() const
{
	FScopeLock Lock(&ServicesLock);
	return DiscoveredServices;
}

// --- Callbacks from JNI (dispatched to game thread) ---

void FNetworkServiceDiscoveryAndroid::HandleServiceFound(const FString& ServiceName, const FString& ServiceType)
{
	FNetworkServiceInfo Info;
	Info.ServiceName = ServiceName;
	Info.ServiceType = ServiceType;

	{
		FScopeLock Lock(&ServicesLock);
		DiscoveredServices.Add(Info);
	}
	OnServiceFoundDelegate.Broadcast(Info);
}

void FNetworkServiceDiscoveryAndroid::HandleServiceLost(const FString& ServiceName, const FString& ServiceType)
{
	FNetworkServiceInfo Info;
	Info.ServiceName = ServiceName;
	Info.ServiceType = ServiceType;

	{
		FScopeLock Lock(&ServicesLock);
		DiscoveredServices.RemoveAll([&ServiceName, &ServiceType](const FNetworkServiceInfo& Existing)
		{
			return Existing.ServiceName == ServiceName && Existing.ServiceType == ServiceType;
		});
	}
	OnServiceLostDelegate.Broadcast(Info);
}

void FNetworkServiceDiscoveryAndroid::HandleServiceResolved(const FNetworkServiceInfo& Service)
{
	{
		FScopeLock Lock(&ServicesLock);
		for (FNetworkServiceInfo& Existing : DiscoveredServices)
		{
			if (Existing.ServiceName == Service.ServiceName && Existing.ServiceType == Service.ServiceType)
			{
				Existing = Service;
				break;
			}
		}
	}
	OnServiceResolvedDelegate.Broadcast(Service);
}

void FNetworkServiceDiscoveryAndroid::HandleServiceRegistered(const FString& ServiceName, const FString& ServiceType, int32 Port)
{
	FNetworkServiceInfo Info;
	Info.ServiceName = ServiceName;
	Info.ServiceType = ServiceType;
	Info.Port = Port;
	Info.bIsResolved = true;
	OnServiceRegisteredDelegate.Broadcast(Info);
}

void FNetworkServiceDiscoveryAndroid::HandleError(const FString& ErrorMessage)
{
	OnDiscoveryErrorDelegate.Broadcast(ErrorMessage);
}

// ============================================================================
//  JNI native method implementations
// ============================================================================

extern "C"
{

JNIEXPORT void JNICALL Java_com_epicgames_unreal_NetworkServiceDiscoveryHelper_nativeOnServiceFound(
	JNIEnv* Env, jclass Clazz, jstring JServiceName, jstring JServiceType)
{
	const char* NameChars = Env->GetStringUTFChars(JServiceName, nullptr);
	const char* TypeChars = Env->GetStringUTFChars(JServiceType, nullptr);
	FString ServiceName = UTF8_TO_TCHAR(NameChars);
	FString ServiceType = UTF8_TO_TCHAR(TypeChars);
	Env->ReleaseStringUTFChars(JServiceName, NameChars);
	Env->ReleaseStringUTFChars(JServiceType, TypeChars);

	AsyncTask(ENamedThreads::GameThread, [ServiceName, ServiceType]()
	{
		if (FNetworkServiceDiscoveryAndroid* Impl = FNetworkServiceDiscoveryAndroid::GetInstance())
		{
			Impl->HandleServiceFound(ServiceName, ServiceType);
		}
	});
}

JNIEXPORT void JNICALL Java_com_epicgames_unreal_NetworkServiceDiscoveryHelper_nativeOnServiceLost(
	JNIEnv* Env, jclass Clazz, jstring JServiceName, jstring JServiceType)
{
	const char* NameChars = Env->GetStringUTFChars(JServiceName, nullptr);
	const char* TypeChars = Env->GetStringUTFChars(JServiceType, nullptr);
	FString ServiceName = UTF8_TO_TCHAR(NameChars);
	FString ServiceType = UTF8_TO_TCHAR(TypeChars);
	Env->ReleaseStringUTFChars(JServiceName, NameChars);
	Env->ReleaseStringUTFChars(JServiceType, TypeChars);

	AsyncTask(ENamedThreads::GameThread, [ServiceName, ServiceType]()
	{
		if (FNetworkServiceDiscoveryAndroid* Impl = FNetworkServiceDiscoveryAndroid::GetInstance())
		{
			Impl->HandleServiceLost(ServiceName, ServiceType);
		}
	});
}

JNIEXPORT void JNICALL Java_com_epicgames_unreal_NetworkServiceDiscoveryHelper_nativeOnServiceResolved(
	JNIEnv* Env, jclass Clazz, jstring JServiceName, jstring JServiceType,
	jstring JHostAddress, jint JPort, jobjectArray JTxtKeys, jobjectArray JTxtValues)
{
	const char* NameChars = Env->GetStringUTFChars(JServiceName, nullptr);
	const char* TypeChars = Env->GetStringUTFChars(JServiceType, nullptr);
	const char* AddrChars = Env->GetStringUTFChars(JHostAddress, nullptr);

	FNetworkServiceInfo Info;
	Info.ServiceName = UTF8_TO_TCHAR(NameChars);
	Info.ServiceType = UTF8_TO_TCHAR(TypeChars);
	Info.Address = UTF8_TO_TCHAR(AddrChars);
	Info.Port = (int32)JPort;
	Info.bIsResolved = true;

	Env->ReleaseStringUTFChars(JServiceName, NameChars);
	Env->ReleaseStringUTFChars(JServiceType, TypeChars);
	Env->ReleaseStringUTFChars(JHostAddress, AddrChars);

	// Extract TXT record
	if (JTxtKeys && JTxtValues)
	{
		int32 Count = Env->GetArrayLength(JTxtKeys);
		for (int32 i = 0; i < Count; i++)
		{
			jstring JKey = (jstring)Env->GetObjectArrayElement(JTxtKeys, i);
			jstring JValue = (jstring)Env->GetObjectArrayElement(JTxtValues, i);

			if (JKey && JValue)
			{
				const char* KeyChars = Env->GetStringUTFChars(JKey, nullptr);
				const char* ValueChars = Env->GetStringUTFChars(JValue, nullptr);

				if (KeyChars && ValueChars)
				{
					Info.TxtRecord.Add(UTF8_TO_TCHAR(KeyChars), UTF8_TO_TCHAR(ValueChars));
				}

				if (KeyChars) Env->ReleaseStringUTFChars(JKey, KeyChars);
				if (ValueChars) Env->ReleaseStringUTFChars(JValue, ValueChars);
			}
			if (JKey) Env->DeleteLocalRef(JKey);
			if (JValue) Env->DeleteLocalRef(JValue);
		}
	}

	AsyncTask(ENamedThreads::GameThread, [Info]()
	{
		if (FNetworkServiceDiscoveryAndroid* Impl = FNetworkServiceDiscoveryAndroid::GetInstance())
		{
			Impl->HandleServiceResolved(Info);
		}
	});
}

JNIEXPORT void JNICALL Java_com_epicgames_unreal_NetworkServiceDiscoveryHelper_nativeOnServiceRegistered(
	JNIEnv* Env, jclass Clazz, jstring JServiceName, jstring JServiceType, jint JPort)
{
	const char* NameChars = Env->GetStringUTFChars(JServiceName, nullptr);
	const char* TypeChars = Env->GetStringUTFChars(JServiceType, nullptr);
	FString ServiceName = UTF8_TO_TCHAR(NameChars);
	FString ServiceType = UTF8_TO_TCHAR(TypeChars);
	int32 Port = (int32)JPort;
	Env->ReleaseStringUTFChars(JServiceName, NameChars);
	Env->ReleaseStringUTFChars(JServiceType, TypeChars);

	AsyncTask(ENamedThreads::GameThread, [ServiceName, ServiceType, Port]()
	{
		if (FNetworkServiceDiscoveryAndroid* Impl = FNetworkServiceDiscoveryAndroid::GetInstance())
		{
			Impl->HandleServiceRegistered(ServiceName, ServiceType, Port);
		}
	});
}

JNIEXPORT void JNICALL Java_com_epicgames_unreal_NetworkServiceDiscoveryHelper_nativeOnError(
	JNIEnv* Env, jclass Clazz, jstring JErrorMessage)
{
	const char* MsgChars = Env->GetStringUTFChars(JErrorMessage, nullptr);
	FString ErrorMessage = UTF8_TO_TCHAR(MsgChars);
	Env->ReleaseStringUTFChars(JErrorMessage, MsgChars);

	AsyncTask(ENamedThreads::GameThread, [ErrorMessage]()
	{
		if (FNetworkServiceDiscoveryAndroid* Impl = FNetworkServiceDiscoveryAndroid::GetInstance())
		{
			Impl->HandleError(ErrorMessage);
		}
	});
}

} // extern "C"

#endif // PLATFORM_ANDROID
