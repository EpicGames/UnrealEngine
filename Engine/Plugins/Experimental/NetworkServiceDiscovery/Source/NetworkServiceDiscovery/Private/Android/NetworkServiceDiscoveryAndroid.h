// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INetworkServiceDiscoveryPlatform.h"

#if PLATFORM_ANDROID

#include <jni.h>

/**
 * Android implementation of network service discovery using NsdManager via JNI.
 */
class FNetworkServiceDiscoveryAndroid : public INetworkServiceDiscoveryPlatform
{
public:
	FNetworkServiceDiscoveryAndroid();
	virtual ~FNetworkServiceDiscoveryAndroid();

	// Non-copyable — JNI global refs are owned exclusively
	FNetworkServiceDiscoveryAndroid(const FNetworkServiceDiscoveryAndroid&) = delete;
	FNetworkServiceDiscoveryAndroid& operator=(const FNetworkServiceDiscoveryAndroid&) = delete;

	// Registration
	virtual bool RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord) override;
	virtual void UnregisterService(const FString& ServiceName) override;
	virtual bool IsServiceRegistered(const FString& ServiceName) const override;

	// Discovery
	virtual bool StartDiscovery(const FString& ServiceType) override;
	virtual void StopDiscovery() override;
	virtual bool IsDiscovering() const override;
	virtual void ResolveService(const FNetworkServiceInfo& Service) override;
	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const override;

	// Called from JNI callbacks (on game thread)
	void HandleServiceFound(const FString& ServiceName, const FString& ServiceType);
	void HandleServiceLost(const FString& ServiceName, const FString& ServiceType);
	void HandleServiceResolved(const FNetworkServiceInfo& Service);
	void HandleServiceRegistered(const FString& ServiceName, const FString& ServiceType, int32 Port);
	void HandleError(const FString& ErrorMessage);

	/** Singleton accessor for JNI callbacks */
	static FNetworkServiceDiscoveryAndroid* GetInstance() { return Instance; }

private:
	bool InitJNI();

	/** JNI references — all default-initialized to null for safety on partial InitJNI failure */
	jclass HelperClass = nullptr;
	jobject HelperInstance = nullptr;
	jmethodID RegisterServiceMethod = nullptr;
	jmethodID UnregisterServiceMethod = nullptr;
	jmethodID StartDiscoveryMethod = nullptr;
	jmethodID StopDiscoveryMethod = nullptr;
	jmethodID ResolveServiceMethod = nullptr;
	jmethodID IsServiceRegisteredMethod = nullptr;
	jmethodID IsDiscoveringMethod = nullptr;

	bool bJNIInitialized = false;

	/** Currently discovered services */
	mutable FCriticalSection ServicesLock;
	TArray<FNetworkServiceInfo> DiscoveredServices;

	static FNetworkServiceDiscoveryAndroid* Instance;
};

#endif // PLATFORM_ANDROID
