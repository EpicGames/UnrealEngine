// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INetworkServiceDiscoveryPlatform.h"

@class FNSDServiceBrowserDelegate;
@class FNSDServiceDelegate;

/**
 * Apple implementation of network service discovery using NSNetServiceBrowser / NSNetService.
 * Works on both iOS and macOS.
 */
class FNetworkServiceDiscoveryApple : public INetworkServiceDiscoveryPlatform
{
public:
	FNetworkServiceDiscoveryApple();
	virtual ~FNetworkServiceDiscoveryApple();

	// Registration
	virtual bool RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord) override;
	virtual void UnregisterService(const FString& ServiceName) override;
	virtual bool IsServiceRegistered(const FString& ServiceName) const override;

	// Discovery
	virtual bool StartDiscovery(const FString& ServiceType) override;
	virtual void StopDiscovery() override;
	virtual bool IsDiscovering() const override;
	virtual void ResolveService(const FNetworkServiceInfo& Service) override;
	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const override;

	// Called from Objective-C delegates
	void HandleServiceFound(const FNetworkServiceInfo& Service);
	void HandleServiceLost(const FNetworkServiceInfo& Service);
	void HandleServiceResolved(const FNetworkServiceInfo& Service);
	void HandleServiceRegistered(const FNetworkServiceInfo& Service);
	void HandleError(const FString& ErrorMessage);

private:
	/** Objective-C delegate for NSNetServiceBrowser callbacks */
	FNSDServiceBrowserDelegate* BrowserDelegate;

	/** Published services keyed by service name */
	TMap<FString, FNSDServiceDelegate*> PublishDelegates;

	/** Currently discovered services */
	mutable FCriticalSection ServicesLock;
	TArray<FNetworkServiceInfo> DiscoveredServices;

	bool bIsDiscovering;
};
