// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/NetworkServiceDiscoveryApple.h"
#include "NetworkServiceDiscoveryModule.h"
#include "Async/Async.h"

// CarbonCore (pulled in by Foundation.h) defines its own 'struct FVector' which
// clashes with UE's 'using FVector = ...' alias. Redirect any use of 'FVector'
// inside the Apple headers to a harmless name so the two declarations never meet.
#pragma push_macro("FVector")
#define FVector FVectorWorkaround

#include "Apple/PreAppleSystemHeaders.h"
#import <Foundation/Foundation.h>
#import <arpa/inet.h>
#import <sys/socket.h>
#import <unistd.h>
#include "Apple/PostAppleSystemHeaders.h"

#undef FVector
#pragma pop_macro("FVector")

// ============================================================================
//  Route-check helper
// ============================================================================

/**
 * Returns true if the kernel has a route to the given address.
 * Uses the standard BSD UDP connect() probe: connecting a UDP socket
 * triggers a route lookup without sending any traffic. If the kernel
 * can find a route, connect() returns 0; otherwise it fails immediately.
 * This is Apple's recommended technique for IPv6 transition support.
 */
static bool CanRouteToAddress(const struct sockaddr* Addr, socklen_t AddrLen)
{
	int Sock = socket(Addr->sa_family, SOCK_DGRAM, 0);
	if (Sock < 0)
	{
		return false;
	}
	int Result = connect(Sock, Addr, AddrLen);
	close(Sock);
	return (Result == 0);
}

// ============================================================================
//  Objective-C delegate for NSNetServiceBrowser (discovery)
// ============================================================================

@interface FNSDServiceBrowserDelegate : NSObject <NSNetServiceBrowserDelegate, NSNetServiceDelegate>
{
	FNetworkServiceDiscoveryApple* Owner;
	NSNetServiceBrowser* Browser;
	NSMutableArray<NSNetService*>* ResolvingServices;
}

- (instancetype)initWithOwner:(FNetworkServiceDiscoveryApple*)InOwner;
- (void)startBrowsingForType:(NSString*)ServiceType;
- (void)stopBrowsing;
- (void)resolveService:(NSString*)ServiceName ofType:(NSString*)ServiceType;

@end

@implementation FNSDServiceBrowserDelegate

- (instancetype)initWithOwner:(FNetworkServiceDiscoveryApple*)InOwner
{
	self = [super init];
	if (self)
	{
		Owner = InOwner;
		Browser = [[NSNetServiceBrowser alloc] init];
		Browser.delegate = self;
		ResolvingServices = [[NSMutableArray alloc] init];
	}
	return self;
}

- (void)dealloc
{
	[self stopBrowsing];
	Browser.delegate = nil;
	[Browser release];
	[ResolvingServices release];
	[super dealloc];
}

- (void)startBrowsingForType:(NSString*)ServiceType
{
	// Strip trailing dot from service type — iOS NSBonjourServices plist entries
	// don't include it, and a mismatch causes silent discovery failure.
	NSString* CleanType = ServiceType;
	if ([CleanType hasSuffix:@"."])
	{
		CleanType = [CleanType substringToIndex:[CleanType length] - 1];
	}

	// Must create and start the browser on the main thread to ensure the
	// run loop delivers delegate callbacks. On iOS, UE's game thread may
	// not pump NSRunLoop.
	dispatch_async(dispatch_get_main_queue(), ^{
		[Browser scheduleInRunLoop:[NSRunLoop mainRunLoop] forMode:NSDefaultRunLoopMode];
		[Browser searchForServicesOfType:CleanType inDomain:@"local."];
	});
}

- (void)stopBrowsing
{
	[Browser stop];
	for (NSNetService* Service in ResolvingServices)
	{
		[Service stop];
		Service.delegate = nil;
	}
	[ResolvingServices removeAllObjects];
}

- (void)resolveService:(NSString*)ServiceName ofType:(NSString*)ServiceType
{
	// Must resolve on the main thread so the run loop delivers delegate callbacks.
	dispatch_async(dispatch_get_main_queue(), ^{
		NSNetService* Service = [[NSNetService alloc] initWithDomain:@"local." type:ServiceType name:ServiceName];
		Service.delegate = self;
		[ResolvingServices addObject:Service];
		[Service scheduleInRunLoop:[NSRunLoop mainRunLoop] forMode:NSDefaultRunLoopMode];
		[Service resolveWithTimeout:10.0];
		[Service release];
	});
}

// --- NSNetServiceBrowserDelegate ---

- (void)netServiceBrowserWillSearch:(NSNetServiceBrowser*)Browser
{
	UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Browser started searching"));
}

- (void)netServiceBrowserDidStopSearch:(NSNetServiceBrowser*)Browser
{
	UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Browser stopped searching"));
}

- (void)netServiceBrowser:(NSNetServiceBrowser*)Browser didFindService:(NSNetService*)Service moreComing:(BOOL)MoreComing
{
	FNetworkServiceInfo Info;
	Info.ServiceName = FString(Service.name);
	Info.ServiceType = FString(Service.type);
	Info.HostName = Service.hostName ? FString(Service.hostName) : FString();

	UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Found service '%s' of type '%s'"), *Info.ServiceName, *Info.ServiceType);

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, Info]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleServiceFound(Info);
		}
	});
}

- (void)netServiceBrowser:(NSNetServiceBrowser*)Browser didRemoveService:(NSNetService*)Service moreComing:(BOOL)MoreComing
{
	FNetworkServiceInfo Info;
	Info.ServiceName = FString(Service.name);
	Info.ServiceType = FString(Service.type);

	UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Lost service '%s'"), *Info.ServiceName);

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, Info]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleServiceLost(Info);
		}
	});
}

- (void)netServiceBrowser:(NSNetServiceBrowser*)Browser didNotSearch:(NSDictionary<NSString*, NSNumber*>*)ErrorDict
{
	NSNumber* ErrorCode = ErrorDict[NSNetServicesErrorCode];
	FString ErrorMsg = FString::Printf(TEXT("Apple: Browse failed with error code %d"), ErrorCode ? [ErrorCode intValue] : -1);

	UE_LOG(LogNetworkServiceDiscovery, Error, TEXT("%s"), *ErrorMsg);

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, ErrorMsg]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleError(ErrorMsg);
		}
	});
}

// --- NSNetServiceDelegate (for resolution) ---

- (void)netServiceDidResolveAddress:(NSNetService*)Service
{
	FNetworkServiceInfo Info;
	Info.ServiceName = FString(Service.name);
	Info.ServiceType = FString(Service.type);
	Info.HostName = Service.hostName ? FString(Service.hostName) : FString();
	Info.Port = (int32)Service.port;
	Info.bIsResolved = true;

	// Pick the best address from the resolved set by probing reachability.
	// NSNetService returns both IPv4 and IPv6 addresses but doesn't tell us
	// which address family the mDNS reply actually came in on. On an IPv6-only
	// network the device may have a non-routable APIPA (169.254.x.x) IPv4
	// address, so blindly preferring IPv4 would pick an unreachable host.
	// Instead we iterate the addresses in the order the system provides them
	// and use the first one the kernel can actually route to (UDP connect probe).
	NSString* FallbackAddrStr = nil;
	for (NSData* AddressData in Service.addresses)
	{
		const struct sockaddr* SockAddr = (const struct sockaddr*)[AddressData bytes];
		socklen_t AddrLen = (socklen_t)[AddressData length];

		if (SockAddr->sa_family == AF_INET)
		{
			const struct sockaddr_in* Addr4 = (const struct sockaddr_in*)SockAddr;
			char AddrBuf[INET_ADDRSTRLEN];
			inet_ntop(AF_INET, &Addr4->sin_addr, AddrBuf, sizeof(AddrBuf));
			NSString* Str = [NSString stringWithUTF8String:AddrBuf];

			bool bRoutable = CanRouteToAddress(SockAddr, AddrLen);
			UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Candidate IPv4 %s — %s"),
				*FString(Str), bRoutable ? TEXT("routable") : TEXT("not routable"));

			if (FallbackAddrStr == nil)
			{
				FallbackAddrStr = Str;
			}
			if (bRoutable)
			{
				Info.Address = FString(Str);
				break;
			}
		}
		else if (SockAddr->sa_family == AF_INET6)
		{
			const struct sockaddr_in6* Addr6 = (const struct sockaddr_in6*)SockAddr;

			// Skip link-local addresses (fe80::) — they require scope IDs
			// and are unreliable for cross-device connections
			if (IN6_IS_ADDR_LINKLOCAL(&Addr6->sin6_addr))
			{
				char AddrBuf[INET6_ADDRSTRLEN];
				inet_ntop(AF_INET6, &Addr6->sin6_addr, AddrBuf, sizeof(AddrBuf));
				UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Candidate IPv6 %s — skipped (link-local)"),
					*FString([NSString stringWithUTF8String:AddrBuf]));
				continue;
			}

			char AddrBuf[INET6_ADDRSTRLEN];
			inet_ntop(AF_INET6, &Addr6->sin6_addr, AddrBuf, sizeof(AddrBuf));
			NSString* Str = [NSString stringWithUTF8String:AddrBuf];

			bool bRoutable = CanRouteToAddress(SockAddr, AddrLen);
			UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Candidate IPv6 %s — %s"),
				*FString(Str), bRoutable ? TEXT("routable") : TEXT("not routable"));

			if (FallbackAddrStr == nil)
			{
				FallbackAddrStr = Str;
			}
			if (bRoutable)
			{
				Info.Address = FString(Str);
				break;
			}
		}
	}

	// If no address was routable (shouldn't happen), use the first candidate
	if (Info.Address.IsEmpty() && FallbackAddrStr != nil)
	{
		UE_LOG(LogNetworkServiceDiscovery, Warning, TEXT("Apple: No routable address found, falling back to %s"), *FString(FallbackAddrStr));
		Info.Address = FString(FallbackAddrStr);
	}

	// Extract TXT record
	NSData* TxtData = Service.TXTRecordData;
	if (TxtData)
	{
		NSDictionary<NSString*, NSData*>* TxtDict = [NSNetService dictionaryFromTXTRecordData:TxtData];
		for (NSString* Key in TxtDict)
		{
			NSData* ValueData = TxtDict[Key];
			NSString* ValueStr = [[NSString alloc] initWithData:ValueData encoding:NSUTF8StringEncoding];
			if (ValueStr)
			{
				Info.TxtRecord.Add(FString(Key), FString(ValueStr));
				[ValueStr release];
			}
		}
	}

	UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Resolved service '%s' -> %s:%d"), *Info.ServiceName, *Info.Address, Info.Port);

	// Remove from resolving list
	[ResolvingServices removeObject:Service];
	Service.delegate = nil;

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, Info]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleServiceResolved(Info);
		}
	});
}

- (void)netService:(NSNetService*)Service didNotResolve:(NSDictionary<NSString*, NSNumber*>*)ErrorDict
{
	NSNumber* ErrorCode = ErrorDict[NSNetServicesErrorCode];
	FString ServiceName = FString(Service.name);
	FString ErrorMsg = FString::Printf(TEXT("Apple: Failed to resolve '%s' (error %d)"), *ServiceName, ErrorCode ? [ErrorCode intValue] : -1);

	UE_LOG(LogNetworkServiceDiscovery, Warning, TEXT("%s"), *ErrorMsg);

	[ResolvingServices removeObject:Service];
	Service.delegate = nil;

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, ErrorMsg]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleError(ErrorMsg);
		}
	});
}

@end

// ============================================================================
//  Objective-C delegate for NSNetService (registration/publishing)
// ============================================================================

@interface FNSDServiceDelegate : NSObject <NSNetServiceDelegate>
{
	FNetworkServiceDiscoveryApple* Owner;
	NSNetService* PublishedService;
}

- (instancetype)initWithOwner:(FNetworkServiceDiscoveryApple*)InOwner;
- (bool)publishServiceWithName:(NSString*)Name type:(NSString*)Type port:(int)Port txtRecord:(NSDictionary<NSString*, NSData*>*)TxtRecord;
- (void)unpublish;

@end

@implementation FNSDServiceDelegate

- (instancetype)initWithOwner:(FNetworkServiceDiscoveryApple*)InOwner
{
	self = [super init];
	if (self)
	{
		Owner = InOwner;
		PublishedService = nil;
	}
	return self;
}

- (void)dealloc
{
	[self unpublish];
	[super dealloc];
}

- (bool)publishServiceWithName:(NSString*)Name type:(NSString*)Type port:(int)Port txtRecord:(NSDictionary<NSString*, NSData*>*)TxtRecord
{
	[self unpublish];

	PublishedService = [[NSNetService alloc] initWithDomain:@"local." type:Type name:Name port:Port];
	if (!PublishedService)
	{
		return false;
	}

	PublishedService.delegate = self;

	if (TxtRecord && TxtRecord.count > 0)
	{
		NSData* TxtData = [NSNetService dataFromTXTRecordDictionary:TxtRecord];
		[PublishedService setTXTRecordData:TxtData];
	}

	[PublishedService publish];
	return true;
}

- (void)unpublish
{
	if (PublishedService)
	{
		[PublishedService stop];
		PublishedService.delegate = nil;
		[PublishedService release];
		PublishedService = nil;
	}
}

// --- NSNetServiceDelegate ---

- (void)netServiceDidPublish:(NSNetService*)Service
{
	FNetworkServiceInfo Info;
	Info.ServiceName = FString(Service.name);
	Info.ServiceType = FString(Service.type);
	Info.Port = (int32)Service.port;
	Info.bIsResolved = true;

	UE_LOG(LogNetworkServiceDiscovery, Log, TEXT("Apple: Published service '%s' on port %d"), *Info.ServiceName, Info.Port);

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, Info]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleServiceRegistered(Info);
		}
	});
}

- (void)netService:(NSNetService*)Service didNotPublish:(NSDictionary<NSString*, NSNumber*>*)ErrorDict
{
	NSNumber* ErrorCode = ErrorDict[NSNetServicesErrorCode];
	FString ErrorMsg = FString::Printf(TEXT("Apple: Failed to publish service (error %d)"), ErrorCode ? [ErrorCode intValue] : -1);

	UE_LOG(LogNetworkServiceDiscovery, Error, TEXT("%s"), *ErrorMsg);

	AsyncTask(ENamedThreads::GameThread, [WeakOwner = Owner, ErrorMsg]()
	{
		if (WeakOwner)
		{
			WeakOwner->HandleError(ErrorMsg);
		}
	});
}

@end

// ============================================================================
//  C++ implementation
// ============================================================================

FNetworkServiceDiscoveryApple::FNetworkServiceDiscoveryApple()
	: BrowserDelegate(nil)
	, bIsDiscovering(false)
{
}

FNetworkServiceDiscoveryApple::~FNetworkServiceDiscoveryApple()
{
	UnregisterService(FString());
	StopDiscovery();
}

// --- Registration ---

bool FNetworkServiceDiscoveryApple::RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord)
{
	// Unregister any existing service with this name
	UnregisterService(ServiceName);

	NSString* Name = ServiceName.GetNSString();
	NSString* Type = ServiceType.GetNSString();

	// Build TXT record dictionary
	NSMutableDictionary<NSString*, NSData*>* TxtDict = nil;
	if (TxtRecord.Num() > 0)
	{
		TxtDict = [NSMutableDictionary dictionaryWithCapacity:TxtRecord.Num()];
		for (const auto& Pair : TxtRecord)
		{
			NSString* Key = Pair.Key.GetNSString();
			NSData* Value = [Pair.Value.GetNSString() dataUsingEncoding:NSUTF8StringEncoding];
			[TxtDict setObject:Value forKey:Key];
		}
	}

	FNSDServiceDelegate* Delegate = [[FNSDServiceDelegate alloc] initWithOwner:this];
	bool bSuccess = [Delegate publishServiceWithName:Name type:Type port:Port txtRecord:TxtDict];
	if (bSuccess)
	{
		PublishDelegates.Add(ServiceName, Delegate);
	}
	else
	{
		[Delegate release];
	}

	return bSuccess;
}

void FNetworkServiceDiscoveryApple::UnregisterService(const FString& ServiceName)
{
	if (ServiceName.IsEmpty())
	{
		// Unregister all
		for (auto& Pair : PublishDelegates)
		{
			[Pair.Value unpublish];
			[Pair.Value release];
		}
		PublishDelegates.Empty();
	}
	else
	{
		FNSDServiceDelegate** Found = PublishDelegates.Find(ServiceName);
		if (Found)
		{
			[*Found unpublish];
			[*Found release];
			PublishDelegates.Remove(ServiceName);
		}
	}
}

bool FNetworkServiceDiscoveryApple::IsServiceRegistered(const FString& ServiceName) const
{
	if (ServiceName.IsEmpty())
	{
		return PublishDelegates.Num() > 0;
	}
	return PublishDelegates.Contains(ServiceName);
}

// --- Discovery ---

bool FNetworkServiceDiscoveryApple::StartDiscovery(const FString& ServiceType)
{
	StopDiscovery();

	BrowserDelegate = [[FNSDServiceBrowserDelegate alloc] initWithOwner:this];
	[BrowserDelegate startBrowsingForType:ServiceType.GetNSString()];
	bIsDiscovering = true;

	return true;
}

void FNetworkServiceDiscoveryApple::StopDiscovery()
{
	if (BrowserDelegate)
	{
		[BrowserDelegate stopBrowsing];
		[BrowserDelegate release];
		BrowserDelegate = nil;
	}
	bIsDiscovering = false;

	FScopeLock Lock(&ServicesLock);
	DiscoveredServices.Empty();
}

bool FNetworkServiceDiscoveryApple::IsDiscovering() const
{
	return bIsDiscovering;
}

void FNetworkServiceDiscoveryApple::ResolveService(const FNetworkServiceInfo& Service)
{
	if (BrowserDelegate)
	{
		[BrowserDelegate resolveService:Service.ServiceName.GetNSString() ofType:Service.ServiceType.GetNSString()];
	}
}

TArray<FNetworkServiceInfo> FNetworkServiceDiscoveryApple::GetDiscoveredServices() const
{
	FScopeLock Lock(&ServicesLock);
	return DiscoveredServices;
}

// --- Callbacks from Objective-C delegates (called on game thread) ---

void FNetworkServiceDiscoveryApple::HandleServiceFound(const FNetworkServiceInfo& Service)
{
	{
		FScopeLock Lock(&ServicesLock);
		DiscoveredServices.Add(Service);
	}
	OnServiceFoundDelegate.Broadcast(Service);
}

void FNetworkServiceDiscoveryApple::HandleServiceLost(const FNetworkServiceInfo& Service)
{
	{
		FScopeLock Lock(&ServicesLock);
		DiscoveredServices.RemoveAll([&Service](const FNetworkServiceInfo& Existing)
		{
			return Existing.ServiceName == Service.ServiceName && Existing.ServiceType == Service.ServiceType;
		});
	}
	OnServiceLostDelegate.Broadcast(Service);
}

void FNetworkServiceDiscoveryApple::HandleServiceResolved(const FNetworkServiceInfo& Service)
{
	{
		FScopeLock Lock(&ServicesLock);
		// Update the existing entry with resolved info
		for (FNetworkServiceInfo& Existing : DiscoveredServices)
		{
			if (Existing.ServiceName == Service.ServiceName && Existing.ServiceType == Service.ServiceType)
			{
				Existing = Service;
				break;
			}
		}
	}
	OnServiceResolvedDelegate.Broadcast(Service);
}

void FNetworkServiceDiscoveryApple::HandleServiceRegistered(const FNetworkServiceInfo& Service)
{
	OnServiceRegisteredDelegate.Broadcast(Service);
}

void FNetworkServiceDiscoveryApple::HandleError(const FString& ErrorMessage)
{
	OnDiscoveryErrorDelegate.Broadcast(ErrorMessage);
}
