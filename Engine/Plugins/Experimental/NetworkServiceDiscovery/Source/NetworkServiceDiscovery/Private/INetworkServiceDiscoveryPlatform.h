// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkServiceDiscoveryTypes.h"

/**
 * Internal platform abstraction for network service discovery.
 * Each platform provides a concrete implementation of this interface.
 */
class INetworkServiceDiscoveryPlatform
{
public:
	virtual ~INetworkServiceDiscoveryPlatform() = default;

	// --- Registration ---
	virtual bool RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord) = 0;
	virtual void UnregisterService(const FString& ServiceName) = 0;
	virtual bool IsServiceRegistered(const FString& ServiceName) const = 0;

	// --- Discovery ---
	virtual bool StartDiscovery(const FString& ServiceType) = 0;
	virtual void StopDiscovery() = 0;
	virtual bool IsDiscovering() const = 0;
	virtual void ResolveService(const FNetworkServiceInfo& Service) = 0;
	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const = 0;

	// --- Delegates ---
	FOnServiceFound OnServiceFoundDelegate;
	FOnServiceLost OnServiceLostDelegate;
	FOnServiceResolved OnServiceResolvedDelegate;
	FOnServiceRegistered OnServiceRegisteredDelegate;
	FOnDiscoveryError OnDiscoveryErrorDelegate;
};
