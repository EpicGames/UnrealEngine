// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkServiceDiscoveryModule.h"
#include "INetworkServiceDiscoveryPlatform.h"
#include "Modules/ModuleManager.h"

// Platform headers
#if PLATFORM_IOS || PLATFORM_MAC
#include "Apple/NetworkServiceDiscoveryApple.h"
#elif PLATFORM_ANDROID
#include "Android/NetworkServiceDiscoveryAndroid.h"
#elif WITH_WINDOWS_DNSSD
#include "Windows/NetworkServiceDiscoveryWindows.h"
#include "Null/NetworkServiceDiscoveryNull.h"
#else
#include "Null/NetworkServiceDiscoveryNull.h"
#endif

DEFINE_LOG_CATEGORY(LogNetworkServiceDiscovery);

void FNetworkServiceDiscoveryModule::StartupModule()
{
	PlatformImpl = CreatePlatformImpl();
	UE_LOGF(LogNetworkServiceDiscovery, Log, "NetworkServiceDiscovery module started.");
}

void FNetworkServiceDiscoveryModule::ShutdownModule()
{
	if (PlatformImpl)
	{
		PlatformImpl->UnregisterService(FString());
		PlatformImpl->StopDiscovery();
		PlatformImpl.Reset();
	}
	UE_LOGF(LogNetworkServiceDiscovery, Log, "NetworkServiceDiscovery module shut down.");
}

TUniquePtr<INetworkServiceDiscoveryPlatform> FNetworkServiceDiscoveryModule::CreatePlatformImpl()
{
#if PLATFORM_IOS || PLATFORM_MAC
	UE_LOGF(LogNetworkServiceDiscovery, Log, "Using Apple (NSNetService) implementation.");
	return MakeUnique<FNetworkServiceDiscoveryApple>();
#elif PLATFORM_ANDROID
	UE_LOGF(LogNetworkServiceDiscovery, Log, "Using Android (NsdManager) implementation.");
	return MakeUnique<FNetworkServiceDiscoveryAndroid>();
#elif WITH_WINDOWS_DNSSD
	if (FNetworkServiceDiscoveryWindows::LoadDnsApi())
	{
		UE_LOGF(LogNetworkServiceDiscovery, Log, "Using Windows (native DNS-SD) implementation.");
		return MakeUnique<FNetworkServiceDiscoveryWindows>();
	}
	else
	{
		UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows DNS-SD APIs not available, falling back to Null implementation.");
		return MakeUnique<FNetworkServiceDiscoveryNull>();
	}
#else
	UE_LOGF(LogNetworkServiceDiscovery, Log, "Using Null implementation - mDNS not supported on this platform.");
	return MakeUnique<FNetworkServiceDiscoveryNull>();
#endif
}

// --- Registration ---

bool FNetworkServiceDiscoveryModule::RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord)
{
	check(PlatformImpl);
	UE_LOGF(LogNetworkServiceDiscovery, Log, "RegisterService: Name='%ls' Type='%ls' Port=%d", *ServiceName, *ServiceType, Port);
	return PlatformImpl->RegisterService(ServiceName, ServiceType, Port, TxtRecord);
}

void FNetworkServiceDiscoveryModule::UnregisterService(const FString& ServiceName)
{
	check(PlatformImpl);
	UE_LOGF(LogNetworkServiceDiscovery, Log, "UnregisterService: Name='%ls'", ServiceName.IsEmpty() ? TEXT("(all)") : *ServiceName);
	PlatformImpl->UnregisterService(ServiceName);
}

bool FNetworkServiceDiscoveryModule::IsServiceRegistered(const FString& ServiceName) const
{
	check(PlatformImpl);
	return PlatformImpl->IsServiceRegistered(ServiceName);
}

// --- Discovery ---

bool FNetworkServiceDiscoveryModule::StartDiscovery(const FString& ServiceType)
{
	check(PlatformImpl);
	UE_LOGF(LogNetworkServiceDiscovery, Log, "StartDiscovery: Type='%ls'", *ServiceType);
	return PlatformImpl->StartDiscovery(ServiceType);
}

void FNetworkServiceDiscoveryModule::StopDiscovery()
{
	check(PlatformImpl);
	UE_LOGF(LogNetworkServiceDiscovery, Log, "StopDiscovery");
	PlatformImpl->StopDiscovery();
}

bool FNetworkServiceDiscoveryModule::IsDiscovering() const
{
	check(PlatformImpl);
	return PlatformImpl->IsDiscovering();
}

void FNetworkServiceDiscoveryModule::ResolveService(const FNetworkServiceInfo& Service)
{
	check(PlatformImpl);
	UE_LOGF(LogNetworkServiceDiscovery, Log, "ResolveService: Name='%ls'", *Service.ServiceName);
	PlatformImpl->ResolveService(Service);
}

TArray<FNetworkServiceInfo> FNetworkServiceDiscoveryModule::GetDiscoveredServices() const
{
	check(PlatformImpl);
	return PlatformImpl->GetDiscoveredServices();
}

// --- Delegates ---

FOnServiceFound& FNetworkServiceDiscoveryModule::OnServiceFound()
{
	check(PlatformImpl);
	return PlatformImpl->OnServiceFoundDelegate;
}

FOnServiceLost& FNetworkServiceDiscoveryModule::OnServiceLost()
{
	check(PlatformImpl);
	return PlatformImpl->OnServiceLostDelegate;
}

FOnServiceResolved& FNetworkServiceDiscoveryModule::OnServiceResolved()
{
	check(PlatformImpl);
	return PlatformImpl->OnServiceResolvedDelegate;
}

FOnServiceRegistered& FNetworkServiceDiscoveryModule::OnServiceRegistered()
{
	check(PlatformImpl);
	return PlatformImpl->OnServiceRegisteredDelegate;
}

FOnDiscoveryError& FNetworkServiceDiscoveryModule::OnDiscoveryError()
{
	check(PlatformImpl);
	return PlatformImpl->OnDiscoveryErrorDelegate;
}

IMPLEMENT_MODULE(FNetworkServiceDiscoveryModule, NetworkServiceDiscovery)
