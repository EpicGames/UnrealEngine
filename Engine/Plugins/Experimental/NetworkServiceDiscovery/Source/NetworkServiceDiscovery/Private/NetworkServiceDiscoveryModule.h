// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INetworkServiceDiscovery.h"

class INetworkServiceDiscoveryPlatform;

DECLARE_LOG_CATEGORY_EXTERN(LogNetworkServiceDiscovery, Log, All);

class FNetworkServiceDiscoveryModule : public INetworkServiceDiscoveryModule
{
public:

	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// INetworkServiceDiscoveryModule - Registration
	virtual bool RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord) override;
	virtual void UnregisterService(const FString& ServiceName = FString()) override;
	virtual bool IsServiceRegistered(const FString& ServiceName = FString()) const override;

	// INetworkServiceDiscoveryModule - Discovery
	virtual bool StartDiscovery(const FString& ServiceType) override;
	virtual void StopDiscovery() override;
	virtual bool IsDiscovering() const override;
	virtual void ResolveService(const FNetworkServiceInfo& Service) override;
	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const override;

	// INetworkServiceDiscoveryModule - Delegates
	virtual FOnServiceFound& OnServiceFound() override;
	virtual FOnServiceLost& OnServiceLost() override;
	virtual FOnServiceResolved& OnServiceResolved() override;
	virtual FOnServiceRegistered& OnServiceRegistered() override;
	virtual FOnDiscoveryError& OnDiscoveryError() override;

private:

	/** Platform-specific implementation */
	TUniquePtr<INetworkServiceDiscoveryPlatform> PlatformImpl;

	/** Create the appropriate platform implementation */
	static TUniquePtr<INetworkServiceDiscoveryPlatform> CreatePlatformImpl();
};
