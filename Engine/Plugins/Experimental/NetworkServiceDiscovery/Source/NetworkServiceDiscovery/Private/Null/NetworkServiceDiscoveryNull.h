// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INetworkServiceDiscoveryPlatform.h"
#include "NetworkServiceDiscoveryModule.h"

/**
 * Null (no-op) implementation for unsupported platforms.
 * All operations log a warning and return failure.
 */
class FNetworkServiceDiscoveryNull : public INetworkServiceDiscoveryPlatform
{
public:

	virtual bool RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord) override
	{
		LogUnsupported();
		return false;
	}

	virtual void UnregisterService(const FString& ServiceName) override {}

	virtual bool IsServiceRegistered(const FString& ServiceName) const override { return false; }

	virtual bool StartDiscovery(const FString& ServiceType) override
	{
		LogUnsupported();
		return false;
	}

	virtual void StopDiscovery() override {}

	virtual bool IsDiscovering() const override { return false; }

	virtual void ResolveService(const FNetworkServiceInfo& Service) override
	{
		LogUnsupported();
	}

	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const override
	{
		return TArray<FNetworkServiceInfo>();
	}

private:

	void LogUnsupported()
	{
		static bool bWarned = false;
		if (!bWarned)
		{
			UE_LOGF(LogNetworkServiceDiscovery, Warning, "mDNS/DNS-SD is not supported on this platform.");
			bWarned = true;
		}
	}
};
