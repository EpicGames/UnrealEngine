// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/NetworkServiceDiscoveryWindows.h"
#include "NetworkServiceDiscoveryModule.h"
#include "Async/Async.h"

#if WITH_WINDOWS_DNSSD

#include "Windows/AllowWindowsPlatformTypes.h"
#include <windns.h>
#include <winsock2.h>
#include <ws2tcpip.h>
#include <iphlpapi.h>
#include "Windows/HideWindowsPlatformTypes.h"

// ============================================================================
//  Dynamic loading of dnsapi.dll
// ============================================================================

// Function pointer typedefs matching the dnsapi.dll exports we use
typedef PDNS_SERVICE_INSTANCE (WINAPI* FnDnsServiceConstructInstance)(
	PCWSTR pServiceName, PCWSTR pHostName, PIP4_ADDRESS pIp4, PIP6_ADDRESS pIp6,
	WORD wPort, WORD wPriority, WORD wWeight, DWORD dwPropertiesCount,
	PCWSTR* keys, PCWSTR* values);
typedef VOID (WINAPI* FnDnsServiceFreeInstance)(PDNS_SERVICE_INSTANCE pInstance);
typedef DWORD (WINAPI* FnDnsServiceRegister)(PDNS_SERVICE_REGISTER_REQUEST pRequest, PDNS_SERVICE_CANCEL pCancel);
typedef DWORD (WINAPI* FnDnsServiceDeRegister)(PDNS_SERVICE_REGISTER_REQUEST pRequest, PDNS_SERVICE_CANCEL pCancel);
typedef DWORD (WINAPI* FnDnsServiceRegisterCancel)(PDNS_SERVICE_CANCEL pCancelHandle);
typedef DWORD (WINAPI* FnDnsServiceBrowse)(PDNS_SERVICE_BROWSE_REQUEST pRequest, PDNS_SERVICE_CANCEL pCancel);
typedef DWORD (WINAPI* FnDnsServiceBrowseCancel)(PDNS_SERVICE_CANCEL pCancelHandle);
typedef DWORD (WINAPI* FnDnsServiceResolve)(PDNS_SERVICE_RESOLVE_REQUEST pRequest, PDNS_SERVICE_CANCEL pCancel);
typedef DWORD (WINAPI* FnDnsServiceResolveCancel)(PDNS_SERVICE_CANCEL pCancelHandle);
typedef VOID (WINAPI* FnDnsRecordListFree)(PDNS_RECORD pRecordList, DNS_FREE_TYPE FreeType);

// Static function pointers - populated by LoadDnsApi()
static FnDnsServiceConstructInstance  pDnsServiceConstructInstance  = nullptr;
static FnDnsServiceFreeInstance       pDnsServiceFreeInstance       = nullptr;
static FnDnsServiceRegister           pDnsServiceRegister           = nullptr;
static FnDnsServiceDeRegister         pDnsServiceDeRegister         = nullptr;
static FnDnsServiceRegisterCancel     pDnsServiceRegisterCancel     = nullptr;
static FnDnsServiceBrowse             pDnsServiceBrowse             = nullptr;
static FnDnsServiceBrowseCancel       pDnsServiceBrowseCancel       = nullptr;
static FnDnsServiceResolve            pDnsServiceResolve            = nullptr;
static FnDnsServiceResolveCancel      pDnsServiceResolveCancel      = nullptr;
static FnDnsRecordListFree            pDnsRecordListFree            = nullptr;

bool FNetworkServiceDiscoveryWindows::bDnsApiLoaded = false;

bool FNetworkServiceDiscoveryWindows::LoadDnsApi()
{
	static bool bAttempted = false;
	if (bAttempted)
	{
		return bDnsApiLoaded;
	}
	bAttempted = true;

	HMODULE hDnsApi = LoadLibraryW(L"dnsapi.dll");
	if (!hDnsApi)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Failed to load dnsapi.dll - DNS-SD will not be available");
		return false;
	}

	#define NSD_LOAD_FUNC(Name) \
		__pragma(warning(suppress: 4191)) /* FARPROC to typed function pointer cast is expected */ \
		p##Name = reinterpret_cast<Fn##Name>(GetProcAddress(hDnsApi, #Name)); \
		if (!p##Name) \
		{ \
			UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: dnsapi.dll missing export '%ls' - DNS-SD will not be available", TEXT(#Name)); \
			FreeLibrary(hDnsApi); \
			return false; \
		}

	NSD_LOAD_FUNC(DnsServiceConstructInstance);
	NSD_LOAD_FUNC(DnsServiceFreeInstance);
	NSD_LOAD_FUNC(DnsServiceRegister);
	NSD_LOAD_FUNC(DnsServiceDeRegister);
	NSD_LOAD_FUNC(DnsServiceRegisterCancel);
	NSD_LOAD_FUNC(DnsServiceBrowse);
	NSD_LOAD_FUNC(DnsServiceBrowseCancel);
	NSD_LOAD_FUNC(DnsServiceResolve);
	NSD_LOAD_FUNC(DnsServiceResolveCancel);
	NSD_LOAD_FUNC(DnsRecordListFree);

	#undef NSD_LOAD_FUNC

	// Intentionally never FreeLibrary - the DLL must stay loaded for the
	// lifetime of the process since async callbacks reference its code.
	bDnsApiLoaded = true;
	UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: dnsapi.dll loaded successfully");
	return true;
}

/** Re-browse interval in seconds for detecting service removals */
static constexpr float ReBrowseIntervalSeconds = 5.0f;

// ============================================================================
//  Dynamic loading of iphlpapi.dll (for GetAdaptersAddresses)
// ============================================================================

// Same dynamic-load pattern as dnsapi.dll above - some Wine configurations may
// ship without iphlpapi or without this specific export. Callers must tolerate
// the load failing and fall back to system-default IPv4 selection.
typedef ULONG (WINAPI* FnGetAdaptersAddresses)(ULONG, ULONG, PVOID, PIP_ADAPTER_ADDRESSES, PULONG);
static FnGetAdaptersAddresses pGetAdaptersAddresses = nullptr;
static bool bIpHlpApiLoaded = false;

static bool LoadIpHlpApi()
{
	static bool bAttempted = false;
	if (bAttempted)
	{
		return bIpHlpApiLoaded;
	}
	bAttempted = true;

	HMODULE hIpHlpApi = LoadLibraryW(L"iphlpapi.dll");
	if (!hIpHlpApi)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Failed to load iphlpapi.dll - mDNS will use system-default IPv4");
		return false;
	}

	PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	pGetAdaptersAddresses = reinterpret_cast<FnGetAdaptersAddresses>(GetProcAddress(hIpHlpApi, "GetAdaptersAddresses"));
	PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	if (!pGetAdaptersAddresses)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: iphlpapi.dll missing 'GetAdaptersAddresses' - mDNS will use system-default IPv4");
		FreeLibrary(hIpHlpApi);
		return false;
	}

	// Intentionally never FreeLibrary - matches dnsapi.dll pattern above.
	bIpHlpApiLoaded = true;
	return true;
}

// ============================================================================
//  Helpers
// ============================================================================

static FString BuildInstanceName(const FString& ServiceName, const FString& ServiceType)
{
	FString CleanType = ServiceType;
	CleanType.RemoveFromEnd(TEXT("."));
	return FString::Printf(TEXT("%s.%s.local"), *ServiceName, *CleanType);
}

/** True if AddressHostOrder is in 169.254.0.0/16 (APIPA / link-local). */
static bool IsLinkLocalIPv4(uint32 AddressHostOrder)
{
	return ((AddressHostOrder >> 16) & 0xFFFFu) == 0xA9FEu;
}

/** True if the 16 IPv6 address bytes start with fe80::/10 (link-local).
 *  Takes a raw byte pointer so it works for both IN6_ADDR (sin6_addr.u.Byte) and
 *  IP6_ADDRESS (IP6Byte) without an overload. */
static bool IsLinkLocalIPv6(const BYTE* AddressBytes16)
{
	return AddressBytes16[0] == 0xFE && (AddressBytes16[1] & 0xC0) == 0x80;
}

/** One LAN-publishable interface: its IfIndex, optional IPv4 and IPv6 to publish, plus a
 *  human description. We register the mDNS service separately on each, bound to its own
 *  IfIndex, so multi-NIC hosts (wired + Wi-Fi to different subnets, multi-VLAN dev boxes)
 *  are reachable from any segment they're physically on. Selection mirrors
 *  FSocketSubsystemWindows::GetLocalAdapterAddresses (physical LAN adapters with DNS-eligible
 *  addresses) plus a link-local carve-out for both families so direct cable connections
 *  (IPv4 APIPA / IPv6 fe80::/10) still publish. v4 subnet dedupe avoids mDNS conflict-rename
 *  when two NICs share a physical LAN. */
struct FPublishableAdapter
{
	ULONG       InterfaceIndex = 0;
	IP4_ADDRESS Address4 = 0;          // network byte order; 0 = no IPv4 publish
	IP6_ADDRESS Address6 = {};         // valid when bHasAddress6 is true
	bool        bHasAddress6 = false;
	FString     Description;
};

/** Returns the list of LAN-publishable adapters with their IPv4 and/or IPv6 addresses.
 *  Empty if iphlpapi is unavailable, enumeration fails, or no usable adapter exists -
 *  callers should fall back to a single registration with nullptr IPv4/IPv6 and
 *  InterfaceIndex=0 (Windows system default) in that case. */
static TArray<FPublishableAdapter> FindPublishableAdapters()
{
	TArray<FPublishableAdapter> Result;

	if (!LoadIpHlpApi())
	{
		return Result;
	}

	ULONG BufLen = 16 * 1024;
	TArray<uint8> Buffer;
	DWORD GaaResult = ERROR_BUFFER_OVERFLOW;
	PIP_ADAPTER_ADDRESSES Adapters = nullptr;
	const ULONG Flags = GAA_FLAG_SKIP_ANYCAST | GAA_FLAG_SKIP_MULTICAST
	                  | GAA_FLAG_SKIP_DNS_SERVER | GAA_FLAG_SKIP_FRIENDLY_NAME;

	for (int32 Attempt = 0; Attempt < 3 && GaaResult == ERROR_BUFFER_OVERFLOW; ++Attempt)
	{
		Buffer.SetNumUninitialized(BufLen);
		Adapters = reinterpret_cast<PIP_ADAPTER_ADDRESSES>(Buffer.GetData());
		GaaResult = pGetAdaptersAddresses(AF_UNSPEC, Flags, nullptr, Adapters, &BufLen);
	}

	if (GaaResult != ERROR_SUCCESS)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Warning, "Windows: GetAdaptersAddresses failed with %u - mDNS will use system-default address", GaaResult);
		return Result;
	}

	// Track IPv4 subnets we've already registered on so two NICs on the same physical LAN
	// (wired + Wi-Fi to the same network) don't both announce and trigger mDNS conflict
	// renaming. Key is host-order network address packed with the prefix length.
	TSet<uint64> SeenIPv4Subnets;

	for (PIP_ADAPTER_ADDRESSES Adapter = Adapters; Adapter != nullptr; Adapter = Adapter->Next)
	{
		// Engine-style adapter-type allowlist (matches FSocketSubsystemWindows::GetLocalAdapterAddresses):
		// only physical wired and Wi-Fi adapters. This excludes IF_TYPE_TUNNEL (TwinGate, Tailscale,
		// OpenVPN), IF_TYPE_PROP_VIRTUAL (Hyper-V vSwitch, Docker, VMware), IF_TYPE_PPP,
		// IF_TYPE_SOFTWARE_LOOPBACK, cellular, etc. by adapter type rather than by address range -
		// more robust than range checks since a VPN that uses a non-CGNAT IP (e.g. 10.x.x.x) is
		// still filtered out as a tunnel.
		if (Adapter->IfType != IF_TYPE_ETHERNET_CSMACD && Adapter->IfType != IF_TYPE_IEEE80211)
		{
			continue;
		}
		if (Adapter->OperStatus != IfOperStatusUp)
		{
			continue;
		}

		// Pre-scan: does this adapter have any DNS-eligible address per family? If yes for v4
		// we'll prefer the routable one and ignore any stale APIPA sitting alongside (otherwise
		// we'd publish both and trip mDNS conflict-rename to "MyService (2)"). The link-local
		// carve-outs (APIPA for v4, fe80::/10 for v6) are only meaningful when the adapter has
		// no routable address of that family.
		bool bHasDnsEligibleV4 = false;
		bool bHasDnsEligibleV6 = false;
		for (PIP_ADAPTER_UNICAST_ADDRESS U = Adapter->FirstUnicastAddress; U != nullptr; U = U->Next)
		{
			if (U->Address.lpSockaddr == nullptr || (U->Flags & IP_ADAPTER_ADDRESS_DNS_ELIGIBLE) == 0)
			{
				continue;
			}
			if (U->Address.lpSockaddr->sa_family == AF_INET)
			{
				bHasDnsEligibleV4 = true;
			}
			else if (U->Address.lpSockaddr->sa_family == AF_INET6)
			{
				bHasDnsEligibleV6 = true;
			}
		}

		FPublishableAdapter Entry;
		Entry.InterfaceIndex = Adapter->IfIndex;
		Entry.Description = Adapter->Description ? FString(Adapter->Description) : FString();
		bool bPickedV4 = false;
		bool bPickedV6 = false;

		for (PIP_ADAPTER_UNICAST_ADDRESS U = Adapter->FirstUnicastAddress; U != nullptr; U = U->Next)
		{
			if (U->Address.lpSockaddr == nullptr)
			{
				continue;
			}
			const bool bDnsEligible = (U->Flags & IP_ADAPTER_ADDRESS_DNS_ELIGIBLE) != 0;

			if (!bPickedV4 && U->Address.lpSockaddr->sa_family == AF_INET)
			{
				const SOCKADDR_IN* Sin = reinterpret_cast<const SOCKADDR_IN*>(U->Address.lpSockaddr);
				const uint32 HostOrder = ntohl(Sin->sin_addr.S_un.S_addr);
				const bool bLinkLocal = IsLinkLocalIPv4(HostOrder);

				// Accept DNS-eligible, or APIPA when the adapter has no DNS-eligible v4 at all.
				if (!bDnsEligible && !(bLinkLocal && !bHasDnsEligibleV4))
				{
					continue;
				}

				// Subnet dedupe across adapters: two physical NICs cabled into the same switch
				// would otherwise both announce and trigger mDNS conflict-rename.
				const uint8 PrefixLen = (U->OnLinkPrefixLength > 0 && U->OnLinkPrefixLength <= 32)
					? U->OnLinkPrefixLength : 32;
				const uint32 Mask = (PrefixLen == 32) ? 0xFFFFFFFFu : ~((1u << (32 - PrefixLen)) - 1u);
				const uint64 SubnetKey = (static_cast<uint64>(HostOrder & Mask) << 8) | PrefixLen;
				bool bAlreadySeen = false;
				SeenIPv4Subnets.Add(SubnetKey, &bAlreadySeen);
				if (bAlreadySeen)
				{
					IN_ADDR Addr;
					Addr.S_un.S_addr = Sin->sin_addr.S_un.S_addr;
					WCHAR AddrStr[INET_ADDRSTRLEN];
					InetNtopW(AF_INET, &Addr, AddrStr, INET_ADDRSTRLEN);
					UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: skipping IPv4 %ls on %ls - subnet already covered by another adapter",
						AddrStr, Adapter->Description ? Adapter->Description : L"");
					// Continue scanning - we may still pick up a v6 on this adapter.
					continue;
				}

				Entry.Address4 = Sin->sin_addr.S_un.S_addr;
				bPickedV4 = true;
			}
			else if (!bPickedV6 && U->Address.lpSockaddr->sa_family == AF_INET6)
			{
				const SOCKADDR_IN6* Sin6 = reinterpret_cast<const SOCKADDR_IN6*>(U->Address.lpSockaddr);
				const bool bLinkLocal = IsLinkLocalIPv6(Sin6->sin6_addr.u.Byte);

				// Accept DNS-eligible, or link-local when the adapter has no DNS-eligible v6 at all.
				// Link-local scope is implicit via Request.InterfaceIndex when we register.
				if (!bDnsEligible && !(bLinkLocal && !bHasDnsEligibleV6))
				{
					continue;
				}

				FMemory::Memcpy(Entry.Address6.IP6Byte, Sin6->sin6_addr.u.Byte, 16);
				Entry.bHasAddress6 = true;
				bPickedV6 = true;
			}
		}

		// Only add the adapter if we have at least one address of either family to publish.
		if (bPickedV4 || bPickedV6)
		{
			Result.Add(MoveTemp(Entry));
		}
	}

	if (Result.Num() == 0)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Warning, "Windows: No usable adapter found - mDNS will use system-default address");
	}

	return Result;
}

// ============================================================================
//  Callback context structs
// ============================================================================

/** Context for register/deregister callbacks - captures the alive flag at call-site time.
 *  For register calls, Generation (unique per FRegistration entry) identifies the specific
 *  registration so the callback can store the actual DNS_SERVICE_INSTANCE (which may have
 *  a different name due to mDNS conflict resolution). One service name can have multiple
 *  entries (one per per-adapter address), so InterfaceIndex alone is not unique - the
 *  match key must be Generation. Stale callbacks (from a cancelled/replaced registration)
 *  are detected by generation mismatch and ignored.
 *  For deregister calls, ServiceInstanceToFree holds the DNS_SERVICE_INSTANCE* to free
 *  once the async deregistration completes (must not be freed before the callback). */
struct FRegisterContext
{
	FNetworkServiceDiscoveryWindows* Self;
	TSharedPtr<TAtomic<bool>> AliveFlag;
	FString ServiceName;
	uint64 Generation = 0;
	FString InterfaceName;       // Adapter description string
	uint32  InterfaceIndex = 0;  // Adapter IfIndex bound at register time - surfaced in the broadcast so
	                             // consumers can disambiguate multi-interface registrations.
	PDNS_SERVICE_INSTANCE ServiceInstanceToFree = nullptr;
	bool bIsDeregister = false;
};

/** Context for browse callbacks - captures the alive flag at call-site time.
 *  Ref-counted via TSharedPtr: the caller holds one ref (BrowseCallbackContext member)
 *  and a heap-allocated TSharedPtr copy is passed as the PVOID context to the Windows API.
 *  The callback extracts and copies the shared pointer, then deletes the heap wrapper.
 *  This ensures the context stays alive regardless of whether DnsServiceBrowseCancel
 *  triggers a final callback or not - whichever side releases last frees the memory. */
struct FBrowseContext
{
	FNetworkServiceDiscoveryWindows* Self;
	TSharedPtr<TAtomic<bool>> AliveFlag;
};

// ============================================================================
//  File-static callbacks
// ============================================================================

static void WINAPI NSD_RegisterCallback(DWORD Status, PVOID Context, PDNS_SERVICE_INSTANCE pInstance)
{
	FRegisterContext* Ctx = static_cast<FRegisterContext*>(Context);
	FNetworkServiceDiscoveryWindows* Self = Ctx->Self;
	TSharedPtr<TAtomic<bool>> AliveFlag = Ctx->AliveFlag;
	FString ServiceName = MoveTemp(Ctx->ServiceName);
	uint64 Generation = Ctx->Generation;
	FString InterfaceName = MoveTemp(Ctx->InterfaceName);
	uint32 InterfaceIndex = Ctx->InterfaceIndex;

	// For deregister calls, free the service instance now that deregistration is complete.
	// This must be deferred until the callback because DnsServiceDeRegister is async -
	// freeing the instance before deregistration completes leaves a ghost service on the
	// network, causing name conflicts (e.g. "MyService (2)") on re-registration.
	bool bIsDeregister = Ctx->bIsDeregister;
	PDNS_SERVICE_INSTANCE FreedInstance = Ctx->ServiceInstanceToFree;
	if (FreedInstance)
	{
		pDnsServiceFreeInstance(FreedInstance);
	}
	delete Ctx;

	if (bIsDeregister)
	{
		// Deregister completion - the system may provide a pInstance copy, free it if present.
		// Some Windows versions hand back the same pointer we passed in via pServiceInstance,
		// in which case it has already been freed above - skip to avoid a double-free.
		// Don't store it or broadcast OnServiceRegistered.
		if (pInstance && pInstance != FreedInstance)
		{
			pDnsServiceFreeInstance(pInstance);
		}
		return;
	}

	if (Status == ERROR_SUCCESS)
	{
		FNetworkServiceInfo Info;
		Info.bIsResolved = true;
		if (pInstance)
		{
			if (pInstance->pszInstanceName)
			{
				Info.ServiceName = FString(pInstance->pszInstanceName);
			}
			Info.Port = pInstance->wPort;
		}
		// Identify the registration by interface, not by address. A single interface can
		// carry multiple addresses (dual-stack v4+v6, multi-subnet) and pInstance fields
		// aren't reliably populated in the register-completion callback anyway. Address
		// stays empty on register broadcasts, matching Android/Apple behaviour.
		Info.InterfaceName = InterfaceName;
		Info.InterfaceIndex = InterfaceIndex;

		UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Service registered '%ls' on %ls", *Info.ServiceName, *Info.InterfaceName);

		// Store the callback's DNS_SERVICE_INSTANCE as ActualInstance - it has the real
		// registered name (which may differ from what we requested due to mDNS conflict
		// resolution). DnsServiceDeRegister needs this actual-name instance to work.
		// Generation is unique per FRegistration entry so it alone identifies the right
		// slot; this matters when one adapter contributes multiple entries (multiple IPv4
		// addresses on different subnets). Matching on InterfaceIndex alone would land
		// every callback on the first entry and leak the others.
		// We own pInstance and must free it if we can't store it.
		AsyncTask(ENamedThreads::GameThread, [Self, AliveFlag, ServiceName, Generation, Info, pInstance]()
		{
			if (AliveFlag->Load())
			{
				bool bStored = false;
				if (!ServiceName.IsEmpty())
				{
					TArray<FNetworkServiceDiscoveryWindows::FRegistration>* RegArray = Self->Registrations.Find(ServiceName);
					if (RegArray)
					{
						for (FNetworkServiceDiscoveryWindows::FRegistration& Reg : *RegArray)
						{
							if (Reg.Generation == Generation)
							{
								// This callback matches the current registration entry
								if (Reg.OriginalInstance)
								{
									pDnsServiceFreeInstance(static_cast<PDNS_SERVICE_INSTANCE>(Reg.OriginalInstance));
									Reg.OriginalInstance = nullptr;
								}
								Reg.ActualInstance = pInstance;
								bStored = true;
								break;
							}
						}
					}
				}
				if (!bStored)
				{
					// Stale callback - the registration was unregistered or replaced before
					// this completion fired. Free the orphaned instance and drop the broadcast,
					// matching the error path's behaviour for the same race - otherwise consumers
					// would see a phantom "registered" event for a service that's already gone.
					if (pInstance)
					{
						pDnsServiceFreeInstance(pInstance);
					}
					return;
				}
				Self->OnServiceRegisteredDelegate.Broadcast(Info);
			}
			else if (pInstance)
			{
				pDnsServiceFreeInstance(pInstance);
			}
		});
	}
	else
	{
		FString ErrorMsg = FString::Printf(TEXT("Windows: Registration failed with error %u"), Status);
		UE_LOGF(LogNetworkServiceDiscovery, Error, "%ls", *ErrorMsg);

		// Drop the broadcast if this error belongs to a registration that has already been
		// replaced or unregistered - otherwise UI/toast consumers see a phantom error against
		// a service that is currently registered cleanly. Match on Generation alone since it
		// is unique per FRegistration entry.
		AsyncTask(ENamedThreads::GameThread, [Self, AliveFlag, ServiceName, Generation, ErrorMsg]()
		{
			if (!AliveFlag->Load())
			{
				return;
			}
			const TArray<FNetworkServiceDiscoveryWindows::FRegistration>* RegArray = Self->Registrations.Find(ServiceName);
			if (RegArray)
			{
				bool bStillCurrent = false;
				for (const FNetworkServiceDiscoveryWindows::FRegistration& Reg : *RegArray)
				{
					if (Reg.Generation == Generation)
					{
						bStillCurrent = true;
						break;
					}
				}
				if (!bStillCurrent)
				{
					return;
				}
			}
			else if (!ServiceName.IsEmpty())
			{
				// Map entry has been removed entirely - the registration is gone.
				return;
			}
			Self->OnDiscoveryErrorDelegate.Broadcast(ErrorMsg);
		});
	}
}

static void WINAPI NSD_BrowseCallback(DWORD Status, PVOID Context, PDNS_RECORD pDnsRecord)
{
	// Extract the shared pointer from the heap wrapper - this bumps the ref count so the
	// FBrowseContext stays alive for the duration of this callback even if the caller
	// (StopDiscovery/OnReBrowseTick) has already released its ref.
	TSharedPtr<FBrowseContext>* pSharedCtx = static_cast<TSharedPtr<FBrowseContext>*>(Context);
	TSharedPtr<FBrowseContext> Ctx = *pSharedCtx;
	delete pSharedCtx;

	FNetworkServiceDiscoveryWindows* Self = Ctx->Self;
	TSharedPtr<TAtomic<bool>> AliveFlag = Ctx->AliveFlag;

	if (Status != ERROR_SUCCESS || !pDnsRecord)
	{
		if (Status != ERROR_SUCCESS && Status != ERROR_CANCELLED)
		{
			FString ErrorMsg = FString::Printf(TEXT("Windows: Browse callback error %u"), Status);
			UE_LOGF(LogNetworkServiceDiscovery, Warning, "%ls", *ErrorMsg);

			AsyncTask(ENamedThreads::GameThread, [Self, AliveFlag, ErrorMsg]()
			{
				if (AliveFlag->Load()) { Self->OnDiscoveryErrorDelegate.Broadcast(ErrorMsg); }
			});
		}
		return;
	}

	// Walk the DNS record list - PTR records point to service instance names
	TArray<FNetworkServiceInfo> FoundServices;
	for (PDNS_RECORD pRecord = pDnsRecord; pRecord != nullptr; pRecord = pRecord->pNext)
	{
		if (pRecord->wType == DNS_TYPE_PTR && pRecord->Data.PTR.pNameHost)
		{
			FNetworkServiceInfo Info;
			Info.ServiceName = FString(pRecord->Data.PTR.pNameHost);
			FoundServices.Add(Info);

			UE_LOGF(LogNetworkServiceDiscovery, Verbose, "Windows: Browse found '%ls'", *Info.ServiceName);
		}
	}

	pDnsRecordListFree(pDnsRecord, DnsFreeRecordList);

	AsyncTask(ENamedThreads::GameThread, [Self, AliveFlag, FoundServices]()
	{
		if (!AliveFlag->Load()) { return; }
		// Track which services were seen in this cycle for removal detection
		for (const FNetworkServiceInfo& Info : FoundServices)
		{
			Self->CurrentBrowseCycleServices.Add(Info.ServiceName);

			bool bAlreadyKnown = false;
			{
				FScopeLock Lock(&Self->ServicesLock);
				for (const FNetworkServiceInfo& Existing : Self->DiscoveredServices)
				{
					if (Existing.ServiceName == Info.ServiceName)
					{
						bAlreadyKnown = true;
						break;
					}
				}
				if (!bAlreadyKnown)
				{
					Self->DiscoveredServices.Add(Info);
				}
			}

			if (!bAlreadyKnown)
			{
				Self->OnServiceFoundDelegate.Broadcast(Info);
			}
		}
	});
}

/** Context for resolve callbacks - carries the owner and the heap-allocated query name */
struct FResolveContext
{
	FNetworkServiceDiscoveryWindows* Self;
	TSharedPtr<TAtomic<bool>> AliveFlag;
	WCHAR* QueryNameBuf;
	PDNS_SERVICE_CANCEL CancelHandle;
};

static void WINAPI NSD_ResolveCallback(DWORD Status, PVOID Context, PDNS_SERVICE_INSTANCE pInstance)
{
	FResolveContext* Ctx = static_cast<FResolveContext*>(Context);
	FNetworkServiceDiscoveryWindows* Self = Ctx->Self;
	TSharedPtr<TAtomic<bool>> AliveFlag = Ctx->AliveFlag;

	// Always clean up context resources regardless of object lifetime
	delete[] Ctx->QueryNameBuf;
	if (AliveFlag->Load())
	{
		FScopeLock Lock(&Self->ResolveLock);
		Self->PendingResolveCancels.Remove(Ctx->CancelHandle);
	}
	delete Ctx->CancelHandle;
	delete Ctx;

	if (Status != ERROR_SUCCESS || !pInstance)
	{
		if (Status != ERROR_SUCCESS && Status != ERROR_CANCELLED)
		{
			FString ErrorMsg = FString::Printf(TEXT("Windows: Resolve failed with error %u"), Status);
			UE_LOGF(LogNetworkServiceDiscovery, Warning, "%ls", *ErrorMsg);

			AsyncTask(ENamedThreads::GameThread, [Self, AliveFlag, ErrorMsg]()
			{
				if (AliveFlag->Load()) { Self->OnDiscoveryErrorDelegate.Broadcast(ErrorMsg); }
			});
		}
		return;
	}

	FNetworkServiceInfo Info;
	Info.ServiceName = pInstance->pszInstanceName ? FString(pInstance->pszInstanceName) : FString();
	Info.HostName = pInstance->pszHostName ? FString(pInstance->pszHostName) : FString();
	Info.Port = pInstance->wPort;
	Info.bIsResolved = true;

	if (pInstance->ip4Address)
	{
		IN_ADDR Addr;
		Addr.S_un.S_addr = *pInstance->ip4Address;
		WCHAR AddrStr[INET_ADDRSTRLEN];
		InetNtopW(AF_INET, &Addr, AddrStr, INET_ADDRSTRLEN);
		Info.Address = FString(AddrStr);
	}
	else if (pInstance->ip6Address)
	{
		WCHAR AddrStr[INET6_ADDRSTRLEN];
		InetNtopW(AF_INET6, pInstance->ip6Address, AddrStr, INET6_ADDRSTRLEN);

		// IPv6 link-local addresses (fe80::/10) require a scope ID for the OS to route
		// them; without it connect() returns "no route to host" because the kernel can't
		// pick an interface. Append "%<ifindex>" so consumers get a usable string. Scoped
		// addresses are standard form per RFC 4007.
		if (IsLinkLocalIPv6(pInstance->ip6Address->IP6Byte) && pInstance->dwInterfaceIndex != 0)
		{
			Info.Address = FString::Printf(TEXT("%ls%%%u"), AddrStr, pInstance->dwInterfaceIndex);
		}
		else
		{
			Info.Address = FString(AddrStr);
		}
	}

	for (DWORD i = 0; i < pInstance->dwPropertyCount; i++)
	{
		if (pInstance->keys[i])
		{
			FString Key = FString(pInstance->keys[i]);
			FString Value = pInstance->values[i] ? FString(pInstance->values[i]) : FString();
			Info.TxtRecord.Add(Key, Value);
		}
	}

	UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Resolved '%ls' -> %ls:%d", *Info.ServiceName, *Info.Address, Info.Port);

	AsyncTask(ENamedThreads::GameThread, [Self, AliveFlag, Info]()
	{
		if (!AliveFlag->Load()) { return; }
		{
			FScopeLock Lock(&Self->ServicesLock);
			for (FNetworkServiceInfo& Existing : Self->DiscoveredServices)
			{
				if (Existing.ServiceName == Info.ServiceName)
				{
					Existing = Info;
					break;
				}
			}
		}
		Self->OnServiceResolvedDelegate.Broadcast(Info);
	});
}

// ============================================================================
//  Construction / Destruction
// ============================================================================

FNetworkServiceDiscoveryWindows::FNetworkServiceDiscoveryWindows()
{
}

FNetworkServiceDiscoveryWindows::~FNetworkServiceDiscoveryWindows()
{
	// Signal all async callbacks that this object is no longer valid
	bIsAlive->Store(false);

	UnregisterService(FString());
	StopDiscovery();

	// Cancel any pending resolves - callbacks may still fire but will
	// see bIsAlive==false and skip accessing this object
	FScopeLock Lock(&ResolveLock);
	for (void* Handle : PendingResolveCancels)
	{
		pDnsServiceResolveCancel(static_cast<PDNS_SERVICE_CANCEL>(Handle));
		// Don't delete here - the cancel triggers the callback which handles cleanup
	}
	PendingResolveCancels.Empty();
}

// ============================================================================
//  Service Registration
// ============================================================================

bool FNetworkServiceDiscoveryWindows::RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord)
{
	UnregisterService(ServiceName);

	FString InstanceName = BuildInstanceName(ServiceName, ServiceType);
	FString HostName = FString::Printf(TEXT("%s.local"), FPlatformProcess::ComputerName());

	// Build TXT record arrays - FStrings own the data, pointer arrays are views for the API
	TArray<FString> TxtKeyStrings;
	TArray<FString> TxtValueStrings;
	for (const auto& Pair : TxtRecord)
	{
		TxtKeyStrings.Add(Pair.Key);
		TxtValueStrings.Add(Pair.Value);
	}

	TArray<PCWSTR> TxtKeyPtrs;
	TArray<PCWSTR> TxtValuePtrs;
	for (int32 i = 0; i < TxtKeyStrings.Num(); i++)
	{
		TxtKeyPtrs.Add(*TxtKeyStrings[i]);
		TxtValuePtrs.Add(*TxtValueStrings[i]);
	}

	// Enumerate every LAN-publishable adapter and register the service on each, bound to
	// its own InterfaceIndex with its own A and/or AAAA record. This way a multi-NIC host
	// (wired + Wi-Fi to different subnets, multi-VLAN dev box, IPv6-only USB-ethernet, etc.)
	// is reachable from any segment it's physically attached to. VPN tunnel adapters are
	// filtered out by IfType so tunnels don't leak their address to LAN peers. Empty list
	// (iphlpapi missing or no usable adapter) is handled by falling back to a single
	// registration with nullptr v4/v6 and InterfaceIndex=0 (Windows system default) so we
	// still publish something on Wine and similar setups.
	TArray<FPublishableAdapter> Adapters = FindPublishableAdapters();

	TArray<FRegistration>& RegArray = Registrations.FindOrAdd(ServiceName);

	auto TryRegisterOnAdapter = [&](PIP4_ADDRESS pIp4, PIP6_ADDRESS pIp6, ULONG InterfaceIndex, const TCHAR* AdapterDescription)
	{
		// Generation increments per FRegistration entry (not per RegisterService call), giving
		// each entry a unique identifier the callback can use to match itself back to a slot.
		const uint64 ThisGeneration = ++RegistrationGeneration;

		PDNS_SERVICE_INSTANCE pInstance = pDnsServiceConstructInstance(
			*InstanceName,
			*HostName,
			pIp4,
			pIp6,
			(WORD)Port,
			0, 0,
			(DWORD)TxtKeyPtrs.Num(),
			TxtKeyPtrs.GetData(),
			TxtValuePtrs.GetData()
		);
		if (!pInstance)
		{
			UE_LOGF(LogNetworkServiceDiscovery, Error, "Windows: DnsServiceConstructInstance failed for adapter ifIndex=%u", InterfaceIndex);
			return;
		}

		PDNS_SERVICE_CANCEL pCancel = new DNS_SERVICE_CANCEL;
		FMemory::Memzero(pCancel, sizeof(DNS_SERVICE_CANCEL));

		DNS_SERVICE_REGISTER_REQUEST Request;
		FMemory::Memzero(&Request, sizeof(Request));
		Request.Version = DNS_QUERY_REQUEST_VERSION1;
		Request.InterfaceIndex = InterfaceIndex;
		Request.pServiceInstance = pInstance;
		Request.pRegisterCompletionCallback = &NSD_RegisterCallback;
		Request.unicastEnabled = false;

		FRegisterContext* RegCtx = new FRegisterContext;
		RegCtx->Self = this;
		RegCtx->AliveFlag = bIsAlive;
		RegCtx->ServiceName = ServiceName;
		RegCtx->Generation = ThisGeneration;
		RegCtx->InterfaceName = AdapterDescription ? FString(AdapterDescription) : FString();
		RegCtx->InterfaceIndex = InterfaceIndex;
		Request.pQueryContext = RegCtx;

		DWORD Result = pDnsServiceRegister(&Request, pCancel);
		if (Result != DNS_REQUEST_PENDING && Result != ERROR_SUCCESS)
		{
			UE_LOGF(LogNetworkServiceDiscovery, Error, "Windows: DnsServiceRegister failed with error %u (adapter ifIndex=%u)", Result, InterfaceIndex);
			pDnsServiceFreeInstance(pInstance);
			delete pCancel;
			delete RegCtx;
			return;
		}

		FRegistration Reg;
		Reg.OriginalInstance = pInstance;
		Reg.ActualInstance = nullptr;
		Reg.CancelHandle = pCancel;
		Reg.Generation = ThisGeneration;
		Reg.InterfaceIndex = InterfaceIndex;
		RegArray.Add(MoveTemp(Reg));

		// Build a single "addresses" string for the log so dual-stack adapters get one line
		// like "192.168.0.53 + [2001:db8::1]". Either side may be absent (v4-only, v6-only,
		// or system-default fallback).
		FString AddressesStr;
		if (pIp4)
		{
			IN_ADDR Addr;
			Addr.S_un.S_addr = *pIp4;
			WCHAR AddrStr[INET_ADDRSTRLEN];
			InetNtopW(AF_INET, &Addr, AddrStr, INET_ADDRSTRLEN);
			AddressesStr = FString(AddrStr);
		}
		if (pIp6)
		{
			IN6_ADDR Addr;
			FMemory::Memcpy(Addr.u.Byte, pIp6->IP6Byte, 16);
			WCHAR AddrStr[INET6_ADDRSTRLEN];
			InetNtopW(AF_INET6, &Addr, AddrStr, INET6_ADDRSTRLEN);
			if (!AddressesStr.IsEmpty())
			{
				AddressesStr += TEXT(" + ");
			}
			AddressesStr += FString::Printf(TEXT("[%ls]"), AddrStr);
		}
		if (AddressesStr.IsEmpty())
		{
			AddressesStr = TEXT("system-default");
		}
		UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Registering '%ls' on %ls (%ls, ifIndex=%u) port %d",
			*InstanceName, *AddressesStr, AdapterDescription ? AdapterDescription : L"", InterfaceIndex, Port);
	};

	if (Adapters.Num() == 0)
	{
		// No adapters enumerated - fall back to one registration with system-default selection.
		TryRegisterOnAdapter(nullptr, nullptr, 0, nullptr);
	}
	else
	{
		for (const FPublishableAdapter& Adapter : Adapters)
		{
			IP4_ADDRESS Ip4Copy = Adapter.Address4;
			IP6_ADDRESS Ip6Copy = Adapter.Address6;
			PIP4_ADDRESS pIp4 = (Adapter.Address4 != 0) ? &Ip4Copy : nullptr;
			PIP6_ADDRESS pIp6 = Adapter.bHasAddress6 ? &Ip6Copy : nullptr;
			TryRegisterOnAdapter(pIp4, pIp6, Adapter.InterfaceIndex, *Adapter.Description);
		}
	}

	if (RegArray.Num() == 0)
	{
		// Every per-adapter register call failed synchronously - clean up the empty map slot.
		Registrations.Remove(ServiceName);
		return false;
	}

	return true;
}

void FNetworkServiceDiscoveryWindows::UnregisterService(const FString& ServiceName)
{
	auto DeregisterOne = [this](FRegistration& Reg)
	{
		if (Reg.ActualInstance)
		{
			// Case A: Registration completed - deregister with the actual instance (correct name).
			// DnsServiceDeRegister matches by instance name, so we must use the instance from
			// the callback which has the real registered name (possibly renamed by conflict resolution).
			// InterfaceIndex must mirror what was used at register time: when we bind a registration
			// to a specific adapter, deregistering with InterfaceIndex=0 fails to match and the live
			// service stays advertised - the next register then sees a name collision and Windows
			// renames it to "MyService (2)", "(3)", etc.
			DNS_SERVICE_REGISTER_REQUEST Request;
			FMemory::Memzero(&Request, sizeof(Request));
			Request.Version = DNS_QUERY_REQUEST_VERSION1;
			Request.InterfaceIndex = Reg.InterfaceIndex;
			Request.pServiceInstance = static_cast<PDNS_SERVICE_INSTANCE>(Reg.ActualInstance);
			Request.pRegisterCompletionCallback = &NSD_RegisterCallback;

			FRegisterContext* DeregCtx = new FRegisterContext;
			DeregCtx->Self = this;
			DeregCtx->AliveFlag = bIsAlive;
			DeregCtx->ServiceInstanceToFree = static_cast<PDNS_SERVICE_INSTANCE>(Reg.ActualInstance);
			DeregCtx->bIsDeregister = true;
			Request.pQueryContext = DeregCtx;

			pDnsServiceDeRegister(&Request, nullptr);
			// Don't free the instance here - DnsServiceDeRegister is async.
			// The callback will free it once deregistration actually completes.
			Reg.ActualInstance = nullptr;
		}
		else if (Reg.OriginalInstance)
		{
			// Case B: Registration still pending - cancel by handle (no name needed).
			// DnsServiceRegisterCancel uses the cancel handle to identify the registration,
			// so it works regardless of what name the system would have assigned.
			if (Reg.CancelHandle)
			{
				pDnsServiceRegisterCancel(static_cast<PDNS_SERVICE_CANCEL>(Reg.CancelHandle));
			}
			pDnsServiceFreeInstance(static_cast<PDNS_SERVICE_INSTANCE>(Reg.OriginalInstance));
			Reg.OriginalInstance = nullptr;
			// Zero the generation so the pending callback (if it still fires) sees a
			// mismatch and discards its pInstance instead of storing it.
			Reg.Generation = 0;
		}
		if (Reg.CancelHandle)
		{
			delete static_cast<PDNS_SERVICE_CANCEL>(Reg.CancelHandle);
			Reg.CancelHandle = nullptr;
		}
	};

	if (ServiceName.IsEmpty())
	{
		for (auto& Pair : Registrations)
		{
			for (FRegistration& Reg : Pair.Value)
			{
				DeregisterOne(Reg);
			}
		}
		Registrations.Empty();
	}
	else
	{
		TArray<FRegistration>* Found = Registrations.Find(ServiceName);
		if (Found)
		{
			for (FRegistration& Reg : *Found)
			{
				DeregisterOne(Reg);
			}
			Registrations.Remove(ServiceName);
		}
	}
}

bool FNetworkServiceDiscoveryWindows::IsServiceRegistered(const FString& ServiceName) const
{
	if (ServiceName.IsEmpty())
	{
		for (const auto& Pair : Registrations)
		{
			if (Pair.Value.Num() > 0)
			{
				return true;
			}
		}
		return false;
	}
	const TArray<FRegistration>* Found = Registrations.Find(ServiceName);
	return Found != nullptr && Found->Num() > 0;
}

// ============================================================================
//  Service Discovery (Browse)
// ============================================================================

bool FNetworkServiceDiscoveryWindows::StartDiscovery(const FString& ServiceType)
{
	StopDiscovery();

	BrowseServiceType = ServiceType;

	// Build query name: "_unrealremote._tcp.local" - stored as member to keep alive for async browse
	FString CleanType = ServiceType;
	CleanType.RemoveFromEnd(TEXT("."));
	BrowseQueryName = FString::Printf(TEXT("%s.local"), *CleanType);

	PDNS_SERVICE_CANCEL pCancel = new DNS_SERVICE_CANCEL;
	FMemory::Memzero(pCancel, sizeof(DNS_SERVICE_CANCEL));

	DNS_SERVICE_BROWSE_REQUEST Request;
	FMemory::Memzero(&Request, sizeof(Request));
	Request.Version = DNS_QUERY_REQUEST_VERSION1;
	Request.InterfaceIndex = 0;
	Request.QueryName = const_cast<PWSTR>(*BrowseQueryName);
	Request.pBrowseCallback = &NSD_BrowseCallback;

	TSharedPtr<FBrowseContext> BrowseCtx = MakeShared<FBrowseContext>();
	BrowseCtx->Self = this;
	BrowseCtx->AliveFlag = bIsAlive;
	Request.pQueryContext = new TSharedPtr<FBrowseContext>(BrowseCtx);

	DWORD Result = pDnsServiceBrowse(&Request, pCancel);

	if (Result != DNS_REQUEST_PENDING && Result != ERROR_SUCCESS)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Windows: DnsServiceBrowse failed with error %u", Result);
		delete pCancel;
		delete static_cast<TSharedPtr<FBrowseContext>*>(Request.pQueryContext);
		return false;
	}

	BrowseCancelHandle = pCancel;
	BrowseCallbackContext = BrowseCtx;
	bIsDiscovering = true;

	// Start periodic re-browse to detect service removals
	// (Windows DnsServiceBrowse does not notify when services disappear)
	ReBrowseTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
		TEXT("NetworkServiceDiscoveryReBrowse"),
		ReBrowseIntervalSeconds,
		[this](float DeltaTime) { return OnReBrowseTick(DeltaTime); });

	UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Started browsing for '%ls'", *BrowseQueryName);
	return true;
}

void FNetworkServiceDiscoveryWindows::StopDiscovery()
{
	if (ReBrowseTickerHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(ReBrowseTickerHandle);
		ReBrowseTickerHandle.Reset();
	}

	if (BrowseCancelHandle)
	{
		pDnsServiceBrowseCancel(static_cast<PDNS_SERVICE_CANCEL>(BrowseCancelHandle));
		delete static_cast<PDNS_SERVICE_CANCEL>(BrowseCancelHandle);
		BrowseCancelHandle = nullptr;
	}
	BrowseCallbackContext.Reset();
	bIsDiscovering = false;
	BrowseServiceType.Empty();
	BrowseQueryName.Empty();
	CurrentBrowseCycleServices.Empty();
	PreviousBrowseCycleServices.Empty();
	bFirstBrowseCycleComplete = false;

	FScopeLock Lock(&ServicesLock);
	DiscoveredServices.Empty();
}

bool FNetworkServiceDiscoveryWindows::IsDiscovering() const
{
	return bIsDiscovering;
}

bool FNetworkServiceDiscoveryWindows::OnReBrowseTick(float DeltaTime)
{
	if (!bIsDiscovering || BrowseServiceType.IsEmpty())
	{
		return true; // keep ticking
	}

	// Swap: the current cycle becomes the previous, start a fresh current set.
	// The previous set contains all services seen in the last completed browse cycle.
	PreviousBrowseCycleServices = MoveTemp(CurrentBrowseCycleServices);
	CurrentBrowseCycleServices.Empty();

	// Only diff after the first full cycle has completed - skip the very first tick
	// where PreviousBrowseCycleServices would be empty and cause false removals.
	if (bFirstBrowseCycleComplete)
	{
		TArray<FNetworkServiceInfo> LostServices;
		{
			FScopeLock Lock(&ServicesLock);
			for (int32 i = DiscoveredServices.Num() - 1; i >= 0; --i)
			{
				if (!PreviousBrowseCycleServices.Contains(DiscoveredServices[i].ServiceName))
				{
					LostServices.Add(DiscoveredServices[i]);
					DiscoveredServices.RemoveAtSwap(i);
				}
			}
		}

		for (const FNetworkServiceInfo& Lost : LostServices)
		{
			UE_LOGF(LogNetworkServiceDiscovery, Log, "Windows: Service lost '%ls' (not seen in re-browse)", *Lost.ServiceName);
			OnServiceLostDelegate.Broadcast(Lost);
		}
	}
	else
	{
		bFirstBrowseCycleComplete = true;
	}

	// Issue a new browse - results will populate CurrentBrowseCycleServices via the callback
	if (BrowseCancelHandle)
	{
		pDnsServiceBrowseCancel(static_cast<PDNS_SERVICE_CANCEL>(BrowseCancelHandle));
		delete static_cast<PDNS_SERVICE_CANCEL>(BrowseCancelHandle);
		BrowseCancelHandle = nullptr;
	}
	BrowseCallbackContext.Reset();

	PDNS_SERVICE_CANCEL pCancel = new DNS_SERVICE_CANCEL;
	FMemory::Memzero(pCancel, sizeof(DNS_SERVICE_CANCEL));

	TSharedPtr<FBrowseContext> BrowseCtx = MakeShared<FBrowseContext>();
	BrowseCtx->Self = this;
	BrowseCtx->AliveFlag = bIsAlive;

	DNS_SERVICE_BROWSE_REQUEST Request;
	FMemory::Memzero(&Request, sizeof(Request));
	Request.Version = DNS_QUERY_REQUEST_VERSION1;
	Request.InterfaceIndex = 0;
	Request.QueryName = const_cast<PWSTR>(*BrowseQueryName);
	Request.pBrowseCallback = &NSD_BrowseCallback;
	Request.pQueryContext = new TSharedPtr<FBrowseContext>(BrowseCtx);

	DWORD Result = pDnsServiceBrowse(&Request, pCancel);
	if (Result == DNS_REQUEST_PENDING || Result == ERROR_SUCCESS)
	{
		BrowseCancelHandle = pCancel;
		BrowseCallbackContext = BrowseCtx;
	}
	else
	{
		delete pCancel;
		delete static_cast<TSharedPtr<FBrowseContext>*>(Request.pQueryContext);
	}

	return true; // keep ticking
}

// ============================================================================
//  Service Resolution
// ============================================================================

void FNetworkServiceDiscoveryWindows::ResolveService(const FNetworkServiceInfo& Service)
{
	// Allocate the query name on the heap - DnsServiceResolve stores the pointer
	// for the async operation's lifetime. Freed in the resolve callback via FResolveContext.
	WCHAR* QueryNameBuf = new WCHAR[Service.ServiceName.Len() + 1];
	FCString::Strncpy(QueryNameBuf, *Service.ServiceName, Service.ServiceName.Len() + 1);

	PDNS_SERVICE_CANCEL pCancel = new DNS_SERVICE_CANCEL;
	FMemory::Memzero(pCancel, sizeof(DNS_SERVICE_CANCEL));

	{
		FScopeLock Lock(&ResolveLock);
		PendingResolveCancels.Add(pCancel);
	}

	FResolveContext* Ctx = new FResolveContext;
	Ctx->Self = this;
	Ctx->AliveFlag = bIsAlive;
	Ctx->QueryNameBuf = QueryNameBuf;
	Ctx->CancelHandle = pCancel;

	DNS_SERVICE_RESOLVE_REQUEST Request;
	FMemory::Memzero(&Request, sizeof(Request));
	Request.Version = DNS_QUERY_REQUEST_VERSION1;
	Request.InterfaceIndex = 0;
	Request.QueryName = QueryNameBuf;
	Request.pResolveCompletionCallback = &NSD_ResolveCallback;
	Request.pQueryContext = Ctx;

	DWORD Result = pDnsServiceResolve(&Request, pCancel);

	if (Result != DNS_REQUEST_PENDING && Result != ERROR_SUCCESS)
	{
		UE_LOGF(LogNetworkServiceDiscovery, Error, "Windows: DnsServiceResolve failed with error %u", Result);

		FScopeLock Lock(&ResolveLock);
		PendingResolveCancels.Remove(pCancel);
		delete pCancel;
		delete[] QueryNameBuf;
		delete Ctx;
	}
}

TArray<FNetworkServiceInfo> FNetworkServiceDiscoveryWindows::GetDiscoveredServices() const
{
	FScopeLock Lock(&ServicesLock);
	return DiscoveredServices;
}

#endif // WITH_WINDOWS_DNSSD
