// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INetworkServiceDiscoveryPlatform.h"
#include "Containers/Ticker.h"

#if WITH_WINDOWS_DNSSD

struct FBrowseContext;

/**
 * Windows implementation of network service discovery using native DNS-SD APIs
 * from windns.h / dnsapi.dll (Windows 10 1803+).
 *
 * dnsapi.dll is loaded dynamically at runtime to avoid hard link-time dependencies
 * and allow graceful fallback on older Windows versions.
 *
 * NOTE: Windows DnsServiceBrowse does not provide service removal notifications.
 * We work around this with periodic re-browsing: each browse cycle collects the
 * current set of services and diffs against the previous set to detect removals.
 */
class FNetworkServiceDiscoveryWindows : public INetworkServiceDiscoveryPlatform
{
public:
	FNetworkServiceDiscoveryWindows();
	virtual ~FNetworkServiceDiscoveryWindows();

	/** Shared flag used by async callbacks to detect if this object is still alive */
	TSharedRef<TAtomic<bool>> bIsAlive = MakeShared<TAtomic<bool>>(true);

	/** Try to load dnsapi.dll and resolve all required function pointers. Returns true on success. */
	static bool LoadDnsApi();

	/** Whether dnsapi.dll was loaded successfully. All operations are no-ops if false. */
	static bool bDnsApiLoaded;

	// Registration
	virtual bool RegisterService(const FString& ServiceName, const FString& ServiceType, int32 Port, const TMap<FString, FString>& TxtRecord) override;
	virtual void UnregisterService(const FString& ServiceName) override;
	virtual bool IsServiceRegistered(const FString& ServiceName) const override;

	// Discovery
	virtual bool StartDiscovery(const FString& ServiceType) override;
	virtual void StopDiscovery() override;
	virtual bool IsDiscovering() const override;
	virtual void ResolveService(const FNetworkServiceInfo& Service) override;
	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const override;

public:
	/** Per-registration state — public for file-static callback access.
	 *  Tracks both the original instance (for cancelling pending registrations) and the
	 *  actual instance from the callback (for deregistering active registrations whose
	 *  name may have been changed by mDNS conflict resolution). */
	struct FRegistration
	{
		void* OriginalInstance = nullptr;  // DNS_SERVICE_INSTANCE* we constructed (for pending cancel)
		void* ActualInstance = nullptr;    // DNS_SERVICE_INSTANCE* from callback (for active deregister)
		void* CancelHandle = nullptr;     // DNS_SERVICE_CANCEL*
		uint64 Generation = 0;            // Unique per-entry id - one service name can have multiple entries (one per
		                                  // adapter address), so InterfaceIndex alone does not identify a single
		                                  // registration. Also used to drop stale callbacks (zeroed on cancel).
		uint32 InterfaceIndex = 0;        // ULONG IfIndex bound at register time - must be mirrored on deregister so Windows can match
	};

	/** Active registrations keyed by service name. One service name produces N entries
	 *  (one per LAN-publishable network interface), each bound to its own adapter, so
	 *  multi-NIC hosts can be reached from any segment they're attached to. */
	TMap<FString, TArray<FRegistration>> Registrations;

private:
	/** Monotonic counter - incremented per FRegistration created (every per-adapter entry
	 *  gets its own unique Generation) so callbacks can identify the exact entry they
	 *  belong to, and stale callbacks for cancelled/replaced registrations are dropped. */
	uint64 RegistrationGeneration = 0;

	/** Browse state */
	void* BrowseCancelHandle = nullptr;  // DNS_SERVICE_CANCEL*
	TSharedPtr<FBrowseContext> BrowseCallbackContext;  // Ref-counted; shared with async browse callback
	bool bIsDiscovering = false;
	FString BrowseServiceType;
	FString BrowseQueryName;  // Kept alive for async DnsServiceBrowse

	/** Periodic re-browse ticker for detecting service removals */
	FTSTicker::FDelegateHandle ReBrowseTickerHandle;
	bool OnReBrowseTick(float DeltaTime);

public:
	/** Public for file-static callback access */
	mutable FCriticalSection ServicesLock;
	TArray<FNetworkServiceInfo> DiscoveredServices;
	/** Double-buffered service sets for removal detection.
	 *  BrowseCallback populates CurrentBrowseCycleServices.
	 *  OnReBrowseTick swaps current→previous, then diffs previous against the new current. */
	TSet<FString> CurrentBrowseCycleServices;
	TSet<FString> PreviousBrowseCycleServices;
	bool bFirstBrowseCycleComplete = false;
	mutable FCriticalSection ResolveLock;
	TArray<void*> PendingResolveCancels;
};

#endif // WITH_WINDOWS_DNSSD
