// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "NetworkServiceDiscoveryTypes.h"

#define UE_API NETWORKSERVICEDISCOVERY_API

/**
 * Module interface for cross-platform mDNS/DNS-SD service discovery.
 *
 * Supports two primary operations:
 *   1. Service Registration (host side) - Advertise a service on the local network
 *   2. Service Discovery (client side) - Browse for and resolve services on the local network
 *
 * Platform implementations:
 *   - Apple (iOS/Mac): NSNetServiceBrowser / NSNetService
 *   - Android: NsdManager
 *   - Windows: Native DNS-SD via windns.h / dnsapi.dll (Windows 10 1803+)
 */
class INetworkServiceDiscoveryModule : public IModuleInterface
{
public:

	static INetworkServiceDiscoveryModule* Get()
	{
		return FModuleManager::LoadModulePtr<INetworkServiceDiscoveryModule>("NetworkServiceDiscovery");
	}

	// ------------------------------------------------------------------
	//  Service Registration (Host side - e.g. editor on Win/Mac)
	// ------------------------------------------------------------------

	/**
	 * Register (advertise) a service on the local network via mDNS.
	 *
	 * @param ServiceName   Human-readable name for this service instance
	 * @param ServiceType   DNS-SD service type, e.g. "_unrealremote._tcp."
	 * @param Port          Port the service is listening on
	 * @param TxtRecord     Optional TXT record key-value metadata
	 * @return true if registration was initiated (actual result comes via OnServiceRegistered/OnDiscoveryError)
	 */
	virtual bool RegisterService(
		const FString& ServiceName,
		const FString& ServiceType,
		int32 Port,
		const TMap<FString, FString>& TxtRecord = TMap<FString, FString>()) = 0;

	/**
	 * Unregister (stop advertising) a previously registered service.
	 *
	 * @param ServiceName   Name of the service to unregister. If empty, unregisters all services.
	 */
	virtual void UnregisterService(const FString& ServiceName = FString()) = 0;

	/** Returns true if the named service (or any service, if empty) is currently registered. */
	virtual bool IsServiceRegistered(const FString& ServiceName = FString()) const = 0;

	// ------------------------------------------------------------------
	//  Service Discovery (Client side - e.g. iOS/Android app)
	// ------------------------------------------------------------------

	/**
	 * Start browsing for services of the given type on the local network.
	 *
	 * @param ServiceType   DNS-SD service type to browse for, e.g. "_unrealremote._tcp."
	 * @return true if browsing was started successfully
	 */
	virtual bool StartDiscovery(const FString& ServiceType) = 0;

	/** Stop browsing for services. */
	virtual void StopDiscovery() = 0;

	/** Returns true if currently browsing for services. */
	virtual bool IsDiscovering() const = 0;

	/**
	 * Resolve a discovered service to obtain its IP address and port.
	 * Result delivered via OnServiceResolved delegate.
	 *
	 * @param Service   The service to resolve (as received from OnServiceFound)
	 */
	virtual void ResolveService(const FNetworkServiceInfo& Service) = 0;

	/** Get a snapshot of all currently discovered services. */
	virtual TArray<FNetworkServiceInfo> GetDiscoveredServices() const = 0;

	// ------------------------------------------------------------------
	//  Delegates
	// ------------------------------------------------------------------

	virtual FOnServiceFound& OnServiceFound() = 0;
	virtual FOnServiceLost& OnServiceLost() = 0;
	virtual FOnServiceResolved& OnServiceResolved() = 0;

	/** Fires once for each successful registration. On platforms that publish the
	 *  service per network interface (e.g. Windows multi-NIC hosts) this may fire
	 *  multiple times for a single RegisterService call - once per interface, with
	 *  FNetworkServiceInfo::InterfaceName / InterfaceIndex identifying which interface
	 *  the broadcast is for so consumers can disambiguate. FNetworkServiceInfo::Address
	 *  is NOT populated on register broadcasts (an interface may carry multiple
	 *  addresses; the address is an implementation detail of the publication). */
	virtual FOnServiceRegistered& OnServiceRegistered() = 0;

	virtual FOnDiscoveryError& OnDiscoveryError() = 0;
};

#undef UE_API
