// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Represents a network service discovered or registered via mDNS/DNS-SD.
 */
struct FNetworkServiceInfo
{
	/** Human-readable service name, e.g. "MyProject on DESKTOP-ABC" */
	FString ServiceName;

	/** DNS-SD service type, e.g. "_unrealremote._tcp." */
	FString ServiceType;

	/** Resolved hostname of the service provider */
	FString HostName;

	/** Resolved IP address (IPv4 or IPv6 string, e.g. "192.168.1.100" or "2001:db8::1") */
	FString Address;

	/** Resolved port number */
	int32 Port = 0;

	/** TXT record key-value pairs carrying service metadata */
	TMap<FString, FString> TxtRecord;

	/** Whether this service has been resolved (Address and Port are valid) */
	bool bIsResolved = false;

	/** Human-readable description of the network interface this broadcast is for.
	 *  Set on OnServiceRegistered broadcasts on platforms that publish per network
	 *  interface (Windows). Empty on browse-side broadcasts and on platforms whose
	 *  NSD APIs don't surface per-interface info (Android, Apple). */
	FString InterfaceName;

	/** Adapter index (Windows IfIndex / Unix ifindex) the registration is bound to.
	 *  Set on OnServiceRegistered broadcasts on platforms that publish per network
	 *  interface (Windows). Zero on browse-side broadcasts and on platforms that don't
	 *  surface a per-interface registration. */
	uint32 InterfaceIndex = 0;

	bool operator==(const FNetworkServiceInfo& Other) const
	{
		return ServiceName == Other.ServiceName && ServiceType == Other.ServiceType;
	}

	friend uint32 GetTypeHash(const FNetworkServiceInfo& Info)
	{
		return HashCombine(GetTypeHash(Info.ServiceName), GetTypeHash(Info.ServiceType));
	}
};

/** Fired when a new service is found during discovery (not yet resolved). */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnServiceFound, const FNetworkServiceInfo& /*Service*/);

/** Fired when a previously discovered service is no longer available. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnServiceLost, const FNetworkServiceInfo& /*Service*/);

/** Fired when a discovered service has been resolved to an address and port. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnServiceResolved, const FNetworkServiceInfo& /*Service*/);

/** Fired when a service has been successfully registered/advertised. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnServiceRegistered, const FNetworkServiceInfo& /*Service*/);

/** Fired when a discovery or registration error occurs. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDiscoveryError, const FString& /*ErrorMessage*/);
